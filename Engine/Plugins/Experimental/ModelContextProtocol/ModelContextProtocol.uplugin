{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Unreal MCP",
	"Description": "Anthropic MCP (Model Context Protocol) server implementation for Unreal Engine.",
	"Category": "Other",
	"NoRedist": true,
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "ModelContextProtocol",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelContextProtocolEngine",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelContextProtocolEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelContextProtocolTests",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelContextProtocolEngineTests",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelContextProtocolEditorTests",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "EngineAssetDefinitions",
			"Enabled": true
		},
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		}
	]
}