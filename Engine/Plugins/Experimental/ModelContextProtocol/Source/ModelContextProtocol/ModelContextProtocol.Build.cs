// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelContextProtocol : ModuleRules
{
	public ModelContextProtocol(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Analytics",
				"AnalyticsET",
				"HTTPServer",
				"JsonUtilities",
				"Json"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject"
			});
	}
}
