// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocol.h"

#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY(LogModelContextProtocol);

UE::ModelContextProtocol::EToolNameValidation UE::ModelContextProtocol::ValidateToolName(const FString& ToolName)
{
	if (ToolName.IsEmpty())
	{
		return EToolNameValidation::Empty;
	}

	if (ToolName.Len() > 128)
	{
		return EToolNameValidation::ExceedsMaxLength;
	}

	for (const TCHAR Character : ToolName)
	{
		const bool bIsAllowed =
			(Character >= TEXT('A') && Character <= TEXT('Z')) ||
			(Character >= TEXT('a') && Character <= TEXT('z')) ||
			(Character >= TEXT('0') && Character <= TEXT('9')) ||
			Character == TEXT('_') ||
			Character == TEXT('-') ||
			Character == TEXT('.');

		if (!bIsAllowed)
		{
			return EToolNameValidation::InvalidCharacters;
		}
	}

	return EToolNameValidation::Valid;
}

// CVars
namespace UE::ModelContextProtocol
{
	bool bWrapPODResultsInObject = true;
	static FAutoConsoleVariableRef CVarWrapPODResultsInObject(
		TEXT("ModelContextProtocol.WrapPODToolResultsInObject"),
		bWrapPODResultsInObject,
		TEXT("If true, all non-object (Plain Old Data) tool call results will be wrapped in a {\"result\": <result>} JSON object, to ensure compliance with ")
		TEXT("MCP libraries that expect object results.\n")
		TEXT("\te.g: 1.0 -> {\"result\": 1.0}"));

	bool bAudioResultOggFormat = false;
	static FAutoConsoleVariableRef CVarAudioResultOggFormat(
		TEXT("ModelContextProtocol.AudioResultOggFormat"),
		bAudioResultOggFormat,
		TEXT("If true, return audio results as audio/ogg instead of audio/wav."));

	float ProgressIntervalSeconds = 1.0f;
	static FAutoConsoleVariableRef CVarProgressIntervalSeconds(
		TEXT("ModelContextProtocol.ProgressIntervalSeconds"),
		ProgressIntervalSeconds,
		TEXT("If > 0.0, the seconds between sending progress notifications to clients for async tool requests."));

	int32 PaginationPageSize = 0;
	static FAutoConsoleVariableRef CVarPaginationPageSize(
		TEXT("ModelContextProtocol.PaginationPageSize"),
		PaginationPageSize,
		TEXT("Maximum number of items returned per page for list methods (tools/list, resources/list). 0 = disabled (return all items)."));
}
