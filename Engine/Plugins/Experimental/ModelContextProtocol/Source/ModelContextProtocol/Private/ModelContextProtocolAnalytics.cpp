// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolAnalytics.h"
#include "IModelContextProtocolModule.h"
#include "ModelContextProtocol.h"

#include "AnalyticsEventAttribute.h"
#include "Containers/StringConv.h"
#include "Dom/JsonObject.h"
#include "HAL/IConsoleManager.h"
#include "Hash/Blake3.h"

namespace UE::ModelContextProtocol::Analytics
{
	static bool bEnableAnalytics = true;
	static FAutoConsoleVariableRef CVarEnableAnalytics(
		TEXT("ModelContextProtocol.EnableAnalytics"),
		bEnableAnalytics,
		TEXT("If true, MCP analytics events are recorded via the configured IAnalyticsProviderET."));

	FString HashToolIdentifier(const FString& Identifier)
	{
		FTCHARToUTF8 Utf8(*Identifier);
		const FBlake3Hash Hash = FBlake3::HashBuffer(Utf8.Get(), Utf8.Length());
		return LexToString(Hash);
	}

	bool IsToolResultSuccess(const TSharedPtr<FJsonObject>& ResultJson)
	{
		if (!ResultJson.IsValid())
		{
			return false;
		}

		if (!ResultJson->HasField(TEXT("isError")))
		{
			return true;
		}

		// HasTypedField guards against FJsonObject::TryGetBoolField silently coercing strings
		// and numbers via FString::ToBool; the MCP CallToolResult schema types isError as boolean.
		if (!ResultJson->HasTypedField<EJson::Boolean>(TEXT("isError")))
		{
			UE_LOGF(LogModelContextProtocol, Warning,
				"Tool result has non-bool 'isError' field; treating as error.");
			return false;
		}

		bool bIsError = false;
		ResultJson->TryGetBoolField(TEXT("isError"), bIsError);
		return !bIsError;
	}

	FString ParseToolsetName(const FString& FullToolName)
	{
		FString ToolsetName;
		FString ToolName;
		FullToolName.Split(TEXT("."), &ToolsetName, &ToolName, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
		return ToolsetName;
	}

	static void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
	{
		if (!bEnableAnalytics)
		{
			UE_LOGF(LogModelContextProtocol, VeryVerbose, "Analytics disabled (ModelContextProtocol.EnableAnalytics=false); skipping event '%ls'", *EventName);
			return;
		}

		IModelContextProtocolModule* Module = IModelContextProtocolModule::Get();
		if (!Module)
		{
			return;
		}

		if (!Module->GetAnalyticsProvider().IsValid())
		{
			UE_LOGF(LogModelContextProtocol, VeryVerbose, "No analytics provider set; skipping event '%ls'", *EventName);
			return;
		}

		UE_LOGF(LogModelContextProtocol, VeryVerbose, "Recording analytics event: %ls", *EventName);
		Module->RecordAnalyticsEvent(EventName, Attributes);
	}

	void RecordToolCallEvent(const FString& SessionId, const FString& ToolName, double DurationSeconds, bool bSuccess)
	{
		const FString ToolsetName = ParseToolsetName(ToolName);

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Emplace(TEXT("SessionId"), SessionId);
		Attributes.Emplace(TEXT("ToolsetNameHash"), HashToolIdentifier(ToolsetName));
		Attributes.Emplace(TEXT("ToolNameHash"), HashToolIdentifier(ToolName));
		Attributes.Emplace(TEXT("Duration"), DurationSeconds);
		Attributes.Emplace(TEXT("ResultStatus"), bSuccess ? TEXT("Success") : TEXT("Error"));

		RecordEvent(TEXT("ToolCall"), Attributes);
	}

	void RecordSessionStartEvent(const FString& SessionId, const FString& NegotiatedProtocolVersion)
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Emplace(TEXT("SessionId"), SessionId);
		Attributes.Emplace(TEXT("ProtocolVersion"), NegotiatedProtocolVersion);

		RecordEvent(TEXT("SessionStart"), Attributes);
	}

	void RecordSessionEndEvent(const FString& SessionId)
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Emplace(TEXT("SessionId"), SessionId);

		RecordEvent(TEXT("SessionEnd"), Attributes);
	}
}
