// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolModule.h"
#include "IModelContextProtocolResourceProvider.h"
#include "IModelContextProtocolTool.h"
#include "ModelContextProtocol.h"
#include "ModelContextProtocolServer.h"

#include "AnalyticsEventAttribute.h"
#include "HAL/IConsoleManager.h"
#include "IAnalyticsProviderET.h"

namespace UE::ModelContextProtocol::Private
{
	static constexpr const TCHAR* ModuleName = TEXT("ModelContextProtocol");
}

IMPLEMENT_MODULE(FModelContextProtocolModule, ModelContextProtocol);

namespace UE::ModelContextProtocol::Private
{
	FAutoConsoleCommand CommandRefreshTools = FAutoConsoleCommand(
		TEXT("ModelContextProtocol.RefreshTools"),
		TEXT("Rebuild tools list, releasing any cached schemas etc"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			IModelContextProtocolModule::GetChecked().RefreshTools();
		}),
		ECVF_Cheat);

	FAutoConsoleCommand CommandStartServer = FAutoConsoleCommand(
		TEXT("ModelContextProtocol.StartServer"),
		TEXT("Explicitly starts the MCP HTTP server. Optional: ModelContextProtocol.StartServer <port>"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
			{
				if (Args.Num() > 0)
				{
					const int32 ParsedPort = FCString::Atoi(*Args[0]);
					if (ParsedPort >= 1 && ParsedPort <= 65535)
					{
						Module->StartServer(static_cast<uint32>(ParsedPort));
					}
					else
					{
						UE_LOGF(LogModelContextProtocol, Warning, "Invalid port %d. Must be 1-65535.", ParsedPort);
					}
				}
				else
				{
					Module->StartServer();
				}
			}
		}),
		ECVF_Cheat);

	FAutoConsoleCommand CommandStopServer = FAutoConsoleCommand(
		TEXT("ModelContextProtocol.StopServer"),
		TEXT("Explicitly stops the MCP HTTP server"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
			{
				Module->StopServer();
			}
		}),
		ECVF_Cheat);
}

IModelContextProtocolModule* IModelContextProtocolModule::Get()
{
	return FModuleManager::GetModulePtr<IModelContextProtocolModule>(UE::ModelContextProtocol::Private::ModuleName);
}

IModelContextProtocolModule& IModelContextProtocolModule::GetChecked()
{
	return FModuleManager::LoadModuleChecked<IModelContextProtocolModule>(UE::ModelContextProtocol::Private::ModuleName);
}

const TArray<TSharedRef<IModelContextProtocolTool>>& FModelContextProtocolModule::GetTools() const
{
	return Tools;
}

void FModelContextProtocolModule::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (const TSharedRef<IModelContextProtocolTool>& Tool : Tools)
	{
		Tool->AddReferencedObjects(Collector);
	}
}

FString FModelContextProtocolModule::GetReferencerName() const
{
	return TEXT("FModelContextProtocolModule");
}

TSharedPtr<IModelContextProtocolTool> FModelContextProtocolModule::FindTool(const FString& ToolName) const
{
	const TSharedRef<IModelContextProtocolTool>* Tool = Tools.FindByPredicate(
		[&ToolName](const TSharedPtr<IModelContextProtocolTool>& Tool)
			{
				return Tool->GetName().Equals(ToolName, ESearchCase::IgnoreCase);
			});
	return Tool ? *Tool : TSharedPtr<IModelContextProtocolTool>();
}

bool FModelContextProtocolModule::AddTool(const TSharedRef<IModelContextProtocolTool>& Tool)
{
	const FString ToolName = Tool->GetName();

	if (FindTool(ToolName) != nullptr)
	{
		UE_LOGF(LogModelContextProtocol, Warning, "A ModelContextProtocol tool named \"%ls\" is already registered. All tool names must be globally unique", *ToolName);
		return false;
	}

	switch (UE::ModelContextProtocol::ValidateToolName(ToolName))
	{
	case UE::ModelContextProtocol::EToolNameValidation::Empty:
		UE_LOGF(LogModelContextProtocol, Warning, "Rejecting tool with empty name; MCP spec requires 1-128 characters");
		return false;
	case UE::ModelContextProtocol::EToolNameValidation::ExceedsMaxLength:
		UE_LOGF(LogModelContextProtocol, Warning, "Tool name \"%ls\" exceeds MCP spec limit of 128 characters (%d)", *ToolName, ToolName.Len());
		break;
	case UE::ModelContextProtocol::EToolNameValidation::InvalidCharacters:
		UE_LOGF(LogModelContextProtocol, Log, "Tool name \"%ls\" contains characters outside MCP spec (allowed: A-Z, a-z, 0-9, _, -, .)", *ToolName);
		break;
	default:
		break;
	}

	Tools.Add(Tool);

	return true;
}

bool FModelContextProtocolModule::RemoveTool(const TSharedRef<IModelContextProtocolTool>& Tool)
{
	return Tools.RemoveSingleSwap(Tool) > 0;
}

void FModelContextProtocolModule::RefreshTools()
{
	Tools.Reset();
	OnRefreshToolsDelegate.Broadcast();
}

const TArray<TSharedRef<IModelContextProtocolResourceProvider>>& FModelContextProtocolModule::GetResourceProviders() const
{
	return ResourceProviders;
}

void FModelContextProtocolModule::AddResourceProvider(const TSharedRef<IModelContextProtocolResourceProvider>& ResourceProvider)
{
	ResourceProviders.AddUnique(ResourceProvider);
}

bool FModelContextProtocolModule::RemoveResourceProvider(const TSharedRef<IModelContextProtocolResourceProvider>& ResourceProvider)
{
	return ResourceProviders.RemoveSingleSwap(ResourceProvider) > 0;
}

void FModelContextProtocolModule::SetAnalyticsProvider(const TSharedPtr<IAnalyticsProviderET>& InAnalyticsProvider)
{
	FScopeLock Lock(&AnalyticsMutex);
	AnalyticsProvider = InAnalyticsProvider;
}

TSharedPtr<IAnalyticsProviderET> FModelContextProtocolModule::GetAnalyticsProvider() const
{
	FScopeLock Lock(&AnalyticsMutex);
	return AnalyticsProvider;
}

void FModelContextProtocolModule::SetAnalyticsEventNamespace(const FString& InNamespace)
{
	FScopeLock Lock(&AnalyticsMutex);
	AnalyticsEventNamespace = InNamespace;
}

FString FModelContextProtocolModule::GetAnalyticsEventNamespace() const
{
	FScopeLock Lock(&AnalyticsMutex);
	return AnalyticsEventNamespace;
}

void FModelContextProtocolModule::RecordAnalyticsEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	// Copy the provider and namespace under lock, then release before invoking the provider.
	// Holding the copy keeps the provider alive for the duration of RecordEvent even if another
	// thread calls SetAnalyticsProvider(nullptr) concurrently.
	TSharedPtr<IAnalyticsProviderET> Provider;
	FString Namespace;
	{
		FScopeLock Lock(&AnalyticsMutex);
		Provider = AnalyticsProvider;
		Namespace = AnalyticsEventNamespace;
	}

	if (!Provider.IsValid())
	{
		return;
	}

	FString NamespacedEventName = Namespace.IsEmpty() ? EventName : Namespace + TEXT(".") + EventName;
	Provider->RecordEvent(MoveTemp(NamespacedEventName), Attributes);
}

FModelContextProtocolServer* FModelContextProtocolModule::GetServer()
{
	return Server.Get();
}

void FModelContextProtocolModule::StartServer(uint32 Port, const FString& UrlPath)
{
	if (!Server)
	{
		Server = MakeUnique<FModelContextProtocolServer>();
		OnRefreshToolsDelegateHandle = OnRefreshToolsDelegate.AddRaw(Server.Get(), &FModelContextProtocolServer::ScheduleToolsListChangedBroadcast);
	}
	Server->StartServer(Port, UrlPath);
}

void FModelContextProtocolModule::StopServer()
{
	if (Server)
	{
		OnRefreshToolsDelegate.Remove(OnRefreshToolsDelegateHandle);
		OnRefreshToolsDelegateHandle.Reset();
		Server->StopServer();
		Server.Reset();
	}
}

void FModelContextProtocolModule::StartupModule()
{
	check(Tools.IsEmpty());
	check(ResourceProviders.IsEmpty());
}

void FModelContextProtocolModule::ShutdownModule()
{
	StopServer();
	Tools.Reset();
	ResourceProviders.Reset();
	{
		FScopeLock Lock(&AnalyticsMutex);
		AnalyticsProvider.Reset();
	}
}
