// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IModelContextProtocolModule.h"

#include "HAL/CriticalSection.h"
#include "UObject/GCObject.h"

class FModelContextProtocolServer;
struct IModelContextProtocolResourceProvider;
struct IModelContextProtocolTool;

class FModelContextProtocolModule : public IModelContextProtocolModule, FGCObject
{
public:
	//~ Begin IModelContextProtocolModule implementation
	virtual const TArray<TSharedRef<IModelContextProtocolTool>>& GetTools() const override;
	[[nodiscard]] virtual TSharedPtr<IModelContextProtocolTool> FindTool(const FString& ToolName) const override;
	virtual bool AddTool(const TSharedRef<IModelContextProtocolTool>& Tool) override;
	virtual bool RemoveTool(const TSharedRef<IModelContextProtocolTool>& Tool) override;
	virtual FOnRefreshTools& OnRefreshTools() override { return OnRefreshToolsDelegate; }
	virtual void RefreshTools() override;

	[[nodiscard]] virtual FModelContextProtocolServer* GetServer() override;
	virtual void StartServer(uint32 Port = UE::ModelContextProtocol::DefaultServerPort, const FString& UrlPath = UE::ModelContextProtocol::DefaultServerUrlPath) override;
	virtual void StopServer() override;

	virtual void SetAnalyticsProvider(const TSharedPtr<IAnalyticsProviderET>& InAnalyticsProvider) override;
	[[nodiscard]] virtual TSharedPtr<IAnalyticsProviderET> GetAnalyticsProvider() const override;
	virtual void SetAnalyticsEventNamespace(const FString& InNamespace) override;
	[[nodiscard]] virtual FString GetAnalyticsEventNamespace() const override;
	virtual void RecordAnalyticsEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override;

	virtual const TArray<TSharedRef<IModelContextProtocolResourceProvider>>& GetResourceProviders() const override;
	virtual void AddResourceProvider(const TSharedRef<IModelContextProtocolResourceProvider>& ResourceProvider) override;
	virtual bool RemoveResourceProvider(const TSharedRef<IModelContextProtocolResourceProvider>& ResourceProvider) override;
	//~ End IModelContextProtocolModule implementation

	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface implementation

	//~ Begin FGCObject implementation
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject implementation
protected:

	FOnRefreshTools OnRefreshToolsDelegate;
	TArray<TSharedRef<IModelContextProtocolTool>> Tools;
	TArray<TSharedRef<IModelContextProtocolResourceProvider>> ResourceProviders;
	TUniquePtr<FModelContextProtocolServer> Server;
	FDelegateHandle OnRefreshToolsDelegateHandle;

	/** Guards AnalyticsProvider and AnalyticsEventNamespace. Reads (RecordAnalyticsEvent)
	 *  may occur from any thread via tool-completion callbacks; writes come from setter
	 *  calls which are typically main-thread but not guaranteed. */
	mutable FCriticalSection AnalyticsMutex;
	TSharedPtr<IAnalyticsProviderET> AnalyticsProvider;
	FString AnalyticsEventNamespace = TEXT("ModelContextProtocol");
};
