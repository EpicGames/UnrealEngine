// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolResources.h"

FModelContextProtocolResourceDescriptor::FModelContextProtocolResourceDescriptor(const FString& Uri, const TOptional<FString>& Name,
	const TOptional<FString>& Title, const TOptional<FString>& Description, const TOptional<FString>& MimeType)
{
	JsonResourceObject.Set(TEXT("uri"), Uri);
	if (Name.IsSet())
	{
		JsonResourceObject.Set(TEXT("name"), *Name);
	}
	if (Title.IsSet())
	{
		JsonResourceObject.Set(TEXT("title"), *Title);
	}
	if (Description.IsSet())
	{
		JsonResourceObject.Set(TEXT("description"), *Description);
	}
	if (MimeType.IsSet())
	{
		JsonResourceObject.Set(TEXT("mimeType"), *MimeType);
	}
}

FString FModelContextProtocolResourceDescriptor::GetUri() const
{
	return JsonResourceObject.AsJsonObject()->GetStringField(TEXT("uri"));
}

TSharedRef<FJsonObject> FModelContextProtocolResourceDescriptor::GetJsonObject() const
{
	return JsonResourceObject.AsJsonObject();
}

FModelContextProtocolResource::FModelContextProtocolResource(const FString& Uri, FString&& TextContent, const TOptional<FString>& Name, const TOptional<FString>& Title, const TOptional<FString>& MimeType)
: FModelContextProtocolResourceDescriptor(Uri, Name, Title, /*Description*/{}, MimeType)
{
	JsonResourceObject.Set(TEXT("text"), MoveTemp(TextContent));
}
	
FModelContextProtocolResource::FModelContextProtocolResource(const FString& Uri, const TArrayView<uint8>& BlobContent, const TOptional<FString>& Name, const TOptional<FString>& Title, const TOptional<FString>& MimeType)
: FModelContextProtocolResourceDescriptor(Uri, Name, Title, /*Description*/{}, MimeType)
{
	JsonResourceObject.Set(TEXT("blob"), FBase64::Encode(BlobContent.GetData(), BlobContent.NumBytes()));
}

void FModelContextProtocolResourceDescriptorList::Add(
	const FModelContextProtocolResourceDescriptor& ResourceDescriptor,
	const TSharedRef<const IModelContextProtocolResourceProvider>& ResourceProvider)
{
	// Add resource descriptor to Json array 
	JsonResourceDescriptorArray.Add(ResourceDescriptor.GetJsonObject());

	// Map URI to Provider that produced it
	ResourceUriToProvider.Add(ResourceDescriptor.GetUri(), ResourceProvider);
}

void FModelContextProtocolResourceDescriptorList::Reset()
{
	ReleaseJsonArray();
	ResourceUriToProvider.Reset();
}

int32 FModelContextProtocolResourceDescriptorList::Num() const
{
	return JsonResourceDescriptorArray.Num();
}

TSharedRef<FJsonValueArray> FModelContextProtocolResourceDescriptorList::GetJsonArray() const
{
	return JsonResourceDescriptorArray.AsJsonValue();
}

void FModelContextProtocolResourceDescriptorList::ReleaseJsonArray()
{
	JsonResourceDescriptorArray = FJsonDomBuilder::FArray();
}

TSharedPtr<const IModelContextProtocolResourceProvider> FModelContextProtocolResourceDescriptorList::FindResourceProvider(const FString& Uri) const
{
	if (const TSharedRef<const IModelContextProtocolResourceProvider>* ResourceProvider = ResourceUriToProvider.Find(Uri))
	{
		return *ResourceProvider;
	}
	return nullptr;
}
