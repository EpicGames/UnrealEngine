// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolServer.h"
#include "IModelContextProtocolModule.h"
#include "IModelContextProtocolResourceProvider.h"
#include "IModelContextProtocolTool.h"
#include "ModelContextProtocol.h"
#include "ModelContextProtocolAnalytics.h"

#include "Async/Async.h"
#include "Dom/JsonObject.h"
#include "Misc/Base64.h"
#include "HttpServerModule.h"
#include "IHttpRouter.h"
#include "JsonDomBuilder.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"


namespace UE::ModelContextProtocol::Private
{
	static constexpr const TCHAR* McpSessionIdHeader = TEXT("Mcp-Session-Id");
	static constexpr const TCHAR* McpProtocolVersionHeader = TEXT("Mcp-Protocol-Version");

	/** JSON-RPC method names dispatched by the server. Single source of truth referenced by ProcessJsonRpcCall and GetPostInitMethods. */
	namespace Methods
	{
		static constexpr const TCHAR* Ping = TEXT("ping");
		static constexpr const TCHAR* Initialize = TEXT("initialize");
		static constexpr const TCHAR* NotificationsInitialized = TEXT("notifications/initialized");
		static constexpr const TCHAR* NotificationsCancelled = TEXT("notifications/cancelled");
		static constexpr const TCHAR* ToolsList = TEXT("tools/list");
		static constexpr const TCHAR* ToolsCall = TEXT("tools/call");
		static constexpr const TCHAR* ResourcesList = TEXT("resources/list");
		static constexpr const TCHAR* ResourcesRead = TEXT("resources/read");
	}

	/** Methods that require a resolved session. Anything outside this list (and ping/initialize) is MethodNotFound. */
	const TArray<FString>& GetPostInitMethods()
	{
		static const TArray<FString> MethodNames = {
			Methods::NotificationsInitialized,
			Methods::NotificationsCancelled,
			Methods::ToolsList,
			Methods::ToolsCall,
			Methods::ResourcesList,
			Methods::ResourcesRead,
		};
		return MethodNames;
	}

	bool GetProtocolVersionFromRequestHeaders(const FHttpServerRequest& Request, FString& OutProtocolVersion)
	{
		if (const TArray<FString>* Headers = Request.Headers.Find(McpProtocolVersionHeader))
		{
			if (!Headers->IsEmpty())
			{
				OutProtocolVersion = (*Headers)[0];
				return true;
			}
		}
		return false;
	}

	void CompleteWithResponseCode(const FHttpResultCallback& OnComplete, EHttpServerResponseCodes ResponseCode);

	/**
	 * Validates the Origin header to prevent DNS rebinding attacks per MCP spec.
	 * Allows requests with no Origin (non-browser clients), or Origin matching localhost variants.
	 * Returns true if the request is allowed, false if it was rejected (and OnComplete was called).
	 */
	bool ValidateOriginHeader(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
	{
		const TArray<FString>* OriginHeaders = Request.Headers.Find(TEXT("Origin"));
		if (!OriginHeaders || OriginHeaders->IsEmpty())
		{
			// No Origin header — non-browser client, allow
			return true;
		}

		const FString& Origin = (*OriginHeaders)[0];

		// Extract the host portion from the Origin (scheme://host[:port])
		FString OriginHost;
		FString SchemeAndRest;
		if (Origin.Split(TEXT("://"), &SchemeAndRest, &OriginHost))
		{
			// Strip port if present, handling bracketed IPv6 addresses (e.g., [::1]:8080)
			FString Host;
			if (OriginHost.StartsWith(TEXT("[")))
			{
				int32 CloseBracket;
				if (OriginHost.FindChar(TEXT(']'), CloseBracket))
				{
					Host = OriginHost.Left(CloseBracket + 1);
				}
				else
				{
					Host = OriginHost;
				}
			}
			else
			{
				FString Port;
				if (!OriginHost.Split(TEXT(":"), &Host, &Port))
				{
					Host = OriginHost;
				}
			}

			if (Host == TEXT("localhost") || Host == TEXT("127.0.0.1") || Host == TEXT("[::1]"))
			{
				return true;
			}
		}

		UE_LOGF(LogModelContextProtocol, Warning, "Rejected request with disallowed Origin: %ls", *Origin);
		CompleteWithResponseCode(OnComplete, EHttpServerResponseCodes::Forbidden);
		return false;
	}

	enum class EJsonRpcErrorCode : int32
	{
		/**
		 * Invalid JSON was received by the server.
		 * An error occurred on the server while parsing the JSON text.
		 */
		ParseError = -32700,

		/** The JSON sent is not a valid Request object. */
		InvalidRequest = -32600,

		/** The method does not exist / is not available. */
		MethodNotFound = -32601,

		/** Invalid method parameter(s). */
		InvalidParams = -32602,

		/** Internal JSON-RPC error. */
		InternalError = -32603,

		/** Resource not found (MCP-specific). */
		ResourceNotFound = -32002
	};

	TSharedPtr<FJsonObject> GetJsonObjectFromRequestBody(const TArray<uint8>& InRequestBody)
	{
		const FUtf8StringView IncomingRequestBody(reinterpret_cast<const UTF8CHAR*>(InRequestBody.GetData()), InRequestBody.Num());
		TSharedPtr<FJsonObject> BodyObject = MakeShared<FJsonObject>();
		TSharedRef<TJsonReader<UTF8CHAR>> JsonReader = TJsonReaderFactory<UTF8CHAR>::CreateFromView(IncomingRequestBody);

		if (FJsonSerializer::Deserialize(JsonReader, BodyObject) && BodyObject.IsValid())
		{
			return BodyObject;
		}

		return nullptr;
	}

	TUniquePtr<FHttpServerResponse> CreateJsonRpcErrorResponse(const TSharedPtr<FJsonValue>& RequestId, int32 ErrorCode, const FString& ErrorMessage, EHttpServerResponseCodes ResponseCode = EHttpServerResponseCodes::BadRequest)
	{
		FJsonDomBuilder::FObject ErrorObject;
		ErrorObject.Set(TEXT("code"), ErrorCode);
		ErrorObject.Set(TEXT("message"), ErrorMessage);

		FJsonDomBuilder::FObject ResponseObject;
		ResponseObject.Set(TEXT("jsonrpc"), UE::ModelContextProtocol::JsonRpcVersion);
		ResponseObject.Set(TEXT("id"), RequestId.IsValid() ? RequestId : MakeShared<FJsonValueNull>());
		ResponseObject.Set(TEXT("error"), ErrorObject);

		FUtf8String ResponseStr;
		TSharedRef<TJsonWriter<UTF8CHAR>> JsonWriter = TJsonWriterFactory<UTF8CHAR>::Create(&ResponseStr);
		FJsonSerializer::Serialize(ResponseObject.AsJsonObject(), JsonWriter);

		TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(MoveTemp(ResponseStr), TEXT("application/json"));
		Response->Code = ResponseCode;
		return Response;
	}


	TUniquePtr<FHttpServerResponse> CreateJsonRpcResultResponse(const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonValue>& Result, const FString* SessionId = nullptr)
	{
		if (!ensureMsgf(RequestId.IsValid(), TEXT("JSON-RPC responses *MUST* have a valid request id. Requests without an id must be treated as notifications and should not be replied to.")))
		{
			return CreateJsonRpcErrorResponse(RequestId, static_cast<int32>(EJsonRpcErrorCode::InternalError), TEXT("Replying to request with unknown RequestId"));
		}

		FJsonDomBuilder::FObject ResponseObject;
		ResponseObject.Set(TEXT("jsonrpc"), JsonRpcVersion);
		ResponseObject.Set(TEXT("id"), RequestId);
		ResponseObject.Set(TEXT("result"), Result);

		FUtf8String ResponseStr;
		TSharedRef<TJsonWriter<UTF8CHAR>> JsonWriter = TJsonWriterFactory<UTF8CHAR>::Create(&ResponseStr);
		FJsonSerializer::Serialize(ResponseObject.AsJsonObject(), JsonWriter);

		TUniquePtr<FHttpServerResponse> ServerResponse = FHttpServerResponse::Create(MoveTemp(ResponseStr), TEXT("application/json"));
		if (SessionId)
		{
			ServerResponse->Headers.Add(McpSessionIdHeader, {*SessionId});
		}
		return ServerResponse;
	}

	void CompleteWithError(const FHttpResultCallback& OnComplete, const TSharedPtr<FJsonValue>& RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode ErrorCode, const FString& ErrorMessage, EHttpServerResponseCodes ResponseCode = EHttpServerResponseCodes::BadRequest)
	{
		UE_LOGF(LogModelContextProtocol, Error, "%ls", *ErrorMessage);
		OnComplete(CreateJsonRpcErrorResponse(RequestId, static_cast<int32>(ErrorCode), ErrorMessage, ResponseCode));
	}

	void CompleteWithResult(const FHttpResultCallback& OnComplete, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonValue>& Result, const FString* SessionId = nullptr)
	{
		OnComplete(CreateJsonRpcResultResponse(RequestId, Result, SessionId));
	}

	void CompleteWithResult(const FHttpResultCallback& OnComplete, const TSharedPtr<FJsonValue>& RequestId, const FJsonDomBuilder::FObject& Result, const FString* SessionId = nullptr)
	{
		OnComplete(CreateJsonRpcResultResponse(RequestId, Result.AsJsonValue(), SessionId));
	}

	template <typename ResultType>
	void CompleteWithResult(const FHttpResultCallback& OnComplete, const TSharedPtr<FJsonValue>& RequestId, const ResultType& Result, const FString* SessionId = nullptr)
	{
		TSharedPtr<FJsonObject> JsonResult = FJsonObjectConverter::UStructToJsonObject<ResultType>(Result);
		if (!ensure(JsonResult.IsValid()))
		{
			CompleteWithError(
				OnComplete, RequestId, EJsonRpcErrorCode::InternalError, FString::Printf(TEXT("Error forming response")));
			return;
		}

		TSharedPtr<FJsonValue> JsonResultValue = MakeShared<FJsonValueObject>(JsonResult);
		CompleteWithResult(OnComplete, RequestId, JsonResultValue, SessionId);
	}

	void CompleteWithResponseCode(const FHttpResultCallback& OnComplete, EHttpServerResponseCodes ResponseCode)
	{
		TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
		Response->Code = ResponseCode;
		OnComplete(MoveTemp(Response));
	}

	template <typename ParamsType>
	bool ParseParamsOrError(const TCHAR* MethodName, const TSharedPtr<FJsonValue>& RequestId, const FHttpResultCallback& OnComplete, const TSharedPtr<FJsonObject>& JsonParams, ParamsType& OutParams)
	{
		// Parse params
		if (!JsonParams.IsValid())
		{
			CompleteWithError(
				OnComplete, RequestId, EJsonRpcErrorCode::InvalidParams, FString::Printf(TEXT("Expected params for call to \"%s\""), MethodName));
			return false;
		}
		FText OutFailReason;
		if (!FJsonObjectConverter::JsonObjectToUStruct<ParamsType>(JsonParams.ToSharedRef(), &OutParams, /*CheckFlags*/CPF_None, /*SkipFlags*/CPF_None, /*bStrictMode*/false, &OutFailReason))
		{
			CompleteWithError(
				OnComplete, RequestId, EJsonRpcErrorCode::InvalidParams, FString::Printf(TEXT("Invalid %s params: %s"), MethodName, *OutFailReason.ToString()));
			return false;
		}

		return true;
	}

	bool GetSessionIdFromRequestHeaders(const FHttpServerRequest& Request, FString& OutSessionId)
	{
		if (const TArray<FString>* McpSessionIdHeaders = Request.Headers.Find(McpSessionIdHeader))
		{
			if (!McpSessionIdHeaders->IsEmpty())
			{
				OutSessionId = (*McpSessionIdHeaders)[0];
				return true;
			}
		}
		return false;
	}

	FUtf8String FormatSSEMessage(const FUtf8String& Message)
	{
		return UTF8TEXT("event: message\r\ndata: ") + Message + UTF8TEXT("\r\n\r\n");
	}

	TUniquePtr<FHttpServerResponse> CreateJsonRpcProgressResponse(const TSharedRef<FJsonValue>& ProgressToken, int32 Progress)
	{
		FJsonDomBuilder::FObject ParamsObject;
		ParamsObject.Set(TEXT("progressToken"), ProgressToken);
		ParamsObject.Set(TEXT("progress"), Progress);

		FJsonDomBuilder::FObject ResponseObject;
		ResponseObject.Set(TEXT("jsonrpc"), UE::ModelContextProtocol::JsonRpcVersion);
		ResponseObject.Set(TEXT("method"), TEXT("notifications/progress"));
		ResponseObject.Set(TEXT("params"), ParamsObject);

		FUtf8String ResponseStr;
		TSharedRef<TJsonWriter<UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR>>> JsonWriter = TJsonWriterFactory<UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR>>::Create(&ResponseStr);
		FJsonSerializer::Serialize(ResponseObject.AsJsonObject(), JsonWriter);

		TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(UE::ModelContextProtocol::Private::FormatSSEMessage(ResponseStr), UE::ModelContextProtocol::ContentTypeEventStream);
		Response->Code = EHttpServerResponseCodes::Ok;
		EnumAddFlags(Response->Flags, EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites | EHttpServerResponseFlags::SkipHeaderWrite);

		return Response;
	}

	void SendProgressUpdate(const FHttpResultCallback& ResponseCallback, const TSharedRef<FJsonValue>& ProgressToken, int32 Progress)
	{
		ResponseCallback(CreateJsonRpcProgressResponse(ProgressToken, Progress));
	}

	TUniquePtr<FHttpServerResponse> CreateJsonRpcNotificationResponse(const FString& Method)
	{
		FJsonDomBuilder::FObject NotificationObject;
		NotificationObject.Set(TEXT("jsonrpc"), UE::ModelContextProtocol::JsonRpcVersion);
		NotificationObject.Set(TEXT("method"), Method);
		NotificationObject.Set(TEXT("params"), FJsonDomBuilder::FObject());

		FUtf8String ResponseStr;
		TSharedRef<TJsonWriter<UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR>>> JsonWriter = TJsonWriterFactory<UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR>>::Create(&ResponseStr);
		FJsonSerializer::Serialize(NotificationObject.AsJsonObject(), JsonWriter);

		TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(UE::ModelContextProtocol::Private::FormatSSEMessage(ResponseStr), UE::ModelContextProtocol::ContentTypeEventStream);
		Response->Code = EHttpServerResponseCodes::Ok;
		EnumAddFlags(Response->Flags, EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites | EHttpServerResponseFlags::SkipHeaderWrite);

		return Response;
	}

	struct FPaginationResult
	{
		TArray<TSharedPtr<FJsonValue>> Items;
		TOptional<FString> NextCursor;
	};

	bool ApplyPagination(const TArray<TSharedPtr<FJsonValue>>& AllItems, const TSharedPtr<FJsonObject>& Params, FPaginationResult& OutResult, FString& OutError)
	{
		const int32 PageSize = UE::ModelContextProtocol::PaginationPageSize;

		if (PageSize <= 0)
		{
			OutResult.Items = AllItems;
			return true;
		}

		int32 Offset = 0;

		if (Params.IsValid())
		{
			FString Cursor;
			if (Params->TryGetStringField(TEXT("cursor"), Cursor) && !Cursor.IsEmpty())
			{
				FString DecodedCursor;
				auto IsAllDigits = [](const FString& String)
				{
					for (TCHAR Character : String)
					{
						if (!FChar::IsDigit(Character))
						{
							return false;
						}
					}
					return true;
				};
				if (!FBase64::Decode(Cursor, DecodedCursor) || DecodedCursor.IsEmpty() || !IsAllDigits(DecodedCursor))
				{
					OutError = TEXT("Invalid pagination cursor");
					return false;
				}

				const int64 DecodedOffset = FCString::Atoi64(*DecodedCursor);
				if (DecodedOffset < 0 || DecodedOffset > MAX_int32)
				{
					OutError = TEXT("Invalid pagination cursor");
					return false;
				}
				Offset = static_cast<int32>(DecodedOffset);
			}
		}

		if (Offset > AllItems.Num())
		{
			Offset = AllItems.Num();
		}

		const int32 End = static_cast<int32>(FMath::Min(static_cast<int64>(Offset) + PageSize, static_cast<int64>(AllItems.Num())));

		OutResult.Items.Reserve(End - Offset);
		for (int32 Index = Offset; Index < End; ++Index)
		{
			OutResult.Items.Add(AllItems[Index]);
		}

		if (End < AllItems.Num())
		{
			OutResult.NextCursor = FBase64::Encode(FString::FromInt(End));
		}

		return true;
	}
}

FModelContextProtocolServer::~FModelContextProtocolServer()
{
	AliveGuard.Reset();
	StopServer();
}

TSharedPtr<FModelContextProtocolSession> FModelContextProtocolServer::FindSession(const FString& SessionId) const
{
	const TSharedPtr<FModelContextProtocolSession>* SessionPtr = Sessions.FindByPredicate([&SessionId](const TSharedPtr<FModelContextProtocolSession>& Session)
		{
			return Session->ID == SessionId;
		});
	return SessionPtr ? *SessionPtr : nullptr;
}

void FModelContextProtocolServer::StartServer(uint32 Port, const FString& UrlPath)
{
	const FHttpPath McpServerPath = UrlPath;
	if (!McpServerPath.IsValidPath())
	{
		UE_LOGF(LogModelContextProtocol, Error, "Invalid MCP server URL path '%ls' configured; server will not start. Set a valid path (e.g. '/mcp') in Project Settings -> Plugins -> Model Context Protocol.", *UrlPath);
		return;
	}

	if (HttpRouter.IsValid())
	{
		UE_LOGF(LogModelContextProtocol, Warning, "MCP server is already running on port %u, stopping first.", ActiveServerPort);
		StopServer();
	}

	ActiveServerPort = Port;
	HttpRouter = FHttpServerModule::Get().GetHttpRouter(ActiveServerPort);
	if (ensure(HttpRouter.IsValid()))
	{
		MainMcpRoute = HttpRouter->BindRoute(McpServerPath, EHttpServerRequestVerbs::VERB_POST,
			FHttpRequestHandler::CreateRaw(this, &FModelContextProtocolServer::ProcessPostRequest));

		SseMcpRoute = HttpRouter->BindRoute(McpServerPath, EHttpServerRequestVerbs::VERB_GET,
			FHttpRequestHandler::CreateRaw(this, &FModelContextProtocolServer::ProcessGetRequest));

		DeleteMcpRoute = HttpRouter->BindRoute(McpServerPath, EHttpServerRequestVerbs::VERB_DELETE,
			FHttpRequestHandler::CreateRaw(this, &FModelContextProtocolServer::ProcessDeleteRequest));
	}

	UE_LOGF(LogModelContextProtocol, Log, "Starting MCP server on port %u (override with -ModelContextProtocolPort=N).", ActiveServerPort);
	UE_LOGF(LogModelContextProtocol, Warning, "Data transmitted via this plugin to your connected LLM service is Licensed Technology under the UE EULA. You are responsible for ensuring your LLM provider does not use it as training input. See Section 6(e) of the UE EULA for full terms.");
	FHttpServerModule::Get().StartAllListeners();

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("ModelContextProtocolServer"), 0.0f, [this](float DeltaTime)
	{
		Tick(DeltaTime);
		return true;
	});
}

void FModelContextProtocolServer::StopServer()
{
	for (const TSharedPtr<FModelContextProtocolSession>& Session : Sessions)
	{
		// Only emit SessionEnd for sessions that reached Initialized — SessionStart is emitted from
		// ProcessInitializedNotificationJsonRpcCall, so an Initializing session has no paired Start.
		if (Session.IsValid() && Session->Status == EModelContextProtocolSessionStatus::Initialized)
		{
			UE::ModelContextProtocol::Analytics::RecordSessionEndEvent(Session->ID);
		}
	}

	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}

	if (HttpRouter.IsValid())
	{
		if (MainMcpRoute.IsValid())
		{
			HttpRouter->UnbindRoute(MainMcpRoute);
		}

		if (SseMcpRoute.IsValid())
		{
			HttpRouter->UnbindRoute(SseMcpRoute);
		}

		if (DeleteMcpRoute.IsValid())
		{
			HttpRouter->UnbindRoute(DeleteMcpRoute);
		}
	}

	MainMcpRoute.Reset();
	SseMcpRoute.Reset();
	DeleteMcpRoute.Reset();
	HttpRouter.Reset();
	ActiveServerPort = 0;
}

bool FModelContextProtocolServer::IsServerRunning() const
{
	return HttpRouter.IsValid();
}

uint32 FModelContextProtocolServer::GetServerPort() const
{
	return ActiveServerPort;
}

void FModelContextProtocolServer::ScheduleToolsListChangedBroadcast()
{
	// Defer the actual SSE write until the next Tick. Without this hop, a caller running on the
	// stack of an in-flight tool handler (e.g. `FLoadToolsetTool::Run` invoking `RegisterToolset`)
	// would reenter the HTTP connection state machine mid-write and trip the
	// `AwaitingProcessing == State` assertion in `FHttpConnection::CompleteRead`.
	bToolsListChangedBroadcastScheduled = true;
}

bool FModelContextProtocolServer::ProcessPostRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	if (!UE::ModelContextProtocol::Private::ValidateOriginHeader(Request, OnComplete))
	{
		return true;
	}

	TSharedPtr<FJsonObject> RequestBodyObject = UE::ModelContextProtocol::Private::GetJsonObjectFromRequestBody(Request.Body);
	if (!RequestBodyObject.IsValid())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, /*RequestId*/{}, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::ParseError, TEXT("Invalid JSON body!"));
		return true;
	}

	// jsonrpc
	FString RequestJsonRpcVersion;
	if (!RequestBodyObject->TryGetStringField(TEXT("jsonrpc"), RequestJsonRpcVersion) || RequestJsonRpcVersion != UE::ModelContextProtocol::JsonRpcVersion)
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, /*RequestId*/{}, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidRequest, FString::Printf(TEXT("Unexpected jsonrpc version %s. Must be %s"), *RequestJsonRpcVersion, UE::ModelContextProtocol::JsonRpcVersion));
		return true;
	}

	// id
	TSharedPtr<FJsonValue> RequestId = RequestBodyObject->TryGetField(TEXT("id"));

	// method
	FString Method;
	if (!RequestBodyObject->TryGetStringField(TEXT("method"), Method))
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, /*RequestId*/{}, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidRequest, TEXT("Missing required field \"method\""));
		return true;
	}

	// params
	const TSharedPtr<FJsonObject>* ParamsObject = nullptr;
	RequestBodyObject->TryGetObjectField(TEXT("params"), ParamsObject);

	return ProcessJsonRpcCall(Request, RequestId, Method, ParamsObject ? *ParamsObject : nullptr, OnComplete);
}

bool FModelContextProtocolServer::ProcessJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const FString& Method, const TSharedPtr<FJsonObject>& Params, const FHttpResultCallback& OnComplete)
{
	// ping and initialize are exempt from session validation: ping requires no session and
	// initialize is the request that creates one.
	if (Method == UE::ModelContextProtocol::Private::Methods::Ping)
	{
		return ProcessPingJsonRpcCall(Request, RequestId, OnComplete);
	}
	if (Method == UE::ModelContextProtocol::Private::Methods::Initialize)
	{
		return ProcessInitializeJsonRpcCall(Request, RequestId, Params, OnComplete);
	}

	// Reject unknown methods before session validation so MethodNotFound stays precise:
	// a typo in the method name should not be masked by a session-state error.
	if (!UE::ModelContextProtocol::Private::GetPostInitMethods().Contains(Method))
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::MethodNotFound, FString::Printf(TEXT("Call to unknown method \"%s\""), *Method));
		return true;
	}

	// Per MCP Streamable HTTP spec (2025-06-18 onward): every post-initialize request must
	// carry an Mcp-Session-Id header naming a known session. Missing header is 400; unknown
	// id is 404, which spec-compliant clients treat as a signal to start a new session via
	// initialize.
	FString SessionId;
	if (!UE::ModelContextProtocol::Private::GetSessionIdFromRequestHeaders(Request, SessionId))
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidRequest, FString::Printf(TEXT("Missing required Mcp-Session-Id header for '%s'"), *Method), EHttpServerResponseCodes::BadRequest);
		return true;
	}

	TSharedPtr<FModelContextProtocolSession> Session = FindSession(SessionId);
	if (!Session.IsValid())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidRequest, FString::Printf(TEXT("Unknown session id '%s' for '%s'; client should reinitialize"), *SessionId, *Method), EHttpServerResponseCodes::NotFound);
		return true;
	}

	FString HeaderProtocolVersion;
	if (UE::ModelContextProtocol::Private::GetProtocolVersionFromRequestHeaders(Request, HeaderProtocolVersion)
		&& HeaderProtocolVersion != Session->NegotiatedProtocolVersion)
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidRequest, FString::Printf(TEXT("Mcp-Protocol-Version header '%s' does not match negotiated version '%s'"), *HeaderProtocolVersion, *Session->NegotiatedProtocolVersion));
		return true;
	}

	const TSharedRef<FModelContextProtocolSession> SessionRef = Session.ToSharedRef();

	if (Method == UE::ModelContextProtocol::Private::Methods::NotificationsInitialized)
	{
		return ProcessInitializedNotificationJsonRpcCall(Request, RequestId, Params, SessionRef, OnComplete);
	}
	
	if (Method == UE::ModelContextProtocol::Private::Methods::NotificationsCancelled)
	{
		return ProcessNotificationCancelledJsonRpcCall(Request, RequestId, Params, SessionRef, OnComplete);
	}
	
	if (Method == UE::ModelContextProtocol::Private::Methods::ToolsList)
	{
		return ProcessListToolsJsonRpcCall(Request, RequestId, Params, SessionRef, OnComplete);
	}
	
	if (Method == UE::ModelContextProtocol::Private::Methods::ToolsCall)
	{
		return ProcessToolCallJsonRpcCall(Request, RequestId, Params, SessionRef, OnComplete);
	}
	
	if (Method == UE::ModelContextProtocol::Private::Methods::ResourcesList)
	{
		return ProcessListResourcesJsonRpcCall(Request, RequestId, Params, SessionRef, OnComplete);
	}

	check(Method == UE::ModelContextProtocol::Private::Methods::ResourcesRead);
	
	return ProcessReadResourceJsonRpcCall(Request, RequestId, Params, SessionRef, OnComplete);
}

bool FModelContextProtocolServer::ProcessPingJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const FHttpResultCallback& OnComplete)
{
	// Reply with an empty result per spec for ping.
	UE_LOGF(LogModelContextProtocol, Log, "Replying to Ping");
	FModelContextProtocolPingResult Result;
	UE::ModelContextProtocol::Private::CompleteWithResult(OnComplete, RequestId, Result);
	return true;
}

bool FModelContextProtocolServer::ProcessInitializeJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const FHttpResultCallback& OnComplete)
{
	// initialize is exempt from session validation: any Mcp-Session-Id header on this request is
	// ignored and a brand new session is created. This is the spec-prescribed recovery path for
	// clients that received a 404 from ProcessJsonRpcCall on a stale id.
	FModelContextProtocolInitializeParams InitializeParams;
	if (!UE::ModelContextProtocol::Private::ParseParamsOrError(TEXT("initialize"), RequestId, OnComplete, Params, InitializeParams))
	{
		// Request was completed (with error)
		return true;
	}

	// Negotiate protocol version per MCP spec:
	// If the server supports the client's version, respond with that version.
	// Otherwise, respond with the server's latest supported version.
	const FString NegotiatedVersion = UE::ModelContextProtocol::NegotiateProtocolVersion(InitializeParams.ProtocolVersion);
	UE_LOGF(LogModelContextProtocol, Display, "Client requested protocol version '%ls', negotiated '%ls'", *InitializeParams.ProtocolVersion, *NegotiatedVersion);

	// Create a new session for this client
	TSharedPtr<FModelContextProtocolSession> Session = MakeShared<FModelContextProtocolSession>();
	Session->ID = FGuid::NewGuid().ToString(EGuidFormats::DigitsLower);
	Session->Status = EModelContextProtocolSessionStatus::Initializing;
	Session->NegotiatedProtocolVersion = NegotiatedVersion;
	Session->ClientAddress = Request.PeerAddress;
	Session->ClientCapabilities = InitializeParams.Capabilities;
	Sessions.Add(Session);
	UE_LOGF(LogModelContextProtocol, Log, "Initializing new session: %ls", *Session->ID);

	// Reply with server capabilities
	FModelContextProtocolInitializeResult InitializeResult;
	InitializeResult.ProtocolVersion = NegotiatedVersion;
	FModelContextProtocolToolsCapability ToolsCapability;
	ToolsCapability.ListChanged = true;
	InitializeResult.Capabilities.Tools = ToolsCapability;
	InitializeResult.Capabilities.Resources = FModelContextProtocolResourcesCapability();
	UE::ModelContextProtocol::Private::CompleteWithResult(OnComplete, RequestId, InitializeResult, &Session->ID);
	return true;
}

bool FModelContextProtocolServer::ProcessInitializedNotificationJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete)
{
	Session->Status = EModelContextProtocolSessionStatus::Initialized;
	UE_LOGF(LogModelContextProtocol, Log, "Session initialized: %ls", *Session->ID);

	UE::ModelContextProtocol::Analytics::RecordSessionStartEvent(Session->ID, Session->NegotiatedProtocolVersion);

	// Accept notification
	UE::ModelContextProtocol::Private::CompleteWithResponseCode(OnComplete, EHttpServerResponseCodes::Accepted);
	return true;
}

bool FModelContextProtocolServer::ProcessNotificationCancelledJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete)
{
	if (!Params.IsValid())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, /*RequestId*/nullptr, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidRequest, FString::Printf(TEXT("Expected params for notifications/cancelled for session: %s"), *Session->ID), EHttpServerResponseCodes::BadRequest);
		return true;
	}

	const TSharedPtr<FJsonValue> CancelId = Params->TryGetField(TEXT("requestId"));

	if (!CancelId.IsValid())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, /*RequestId*/nullptr, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidRequest, FString::Printf(TEXT("Expected requestId for notifications/cancelled for session: %s"), *Session->ID), EHttpServerResponseCodes::BadRequest);
		return true;
	}

	FModelContextProtocolToolRequestId ToolRequestId(CancelId.ToSharedRef());

	if (const FModelContextProtocolToolContext* Context = Session->ActiveRequests.Find(ToolRequestId))
	{
		if (Context->Tool.IsValid())
		{
			Context->Tool->CancelAsync(ToolRequestId);
		}

		Session->ActiveRequests.Remove(ToolRequestId);
	}
	// else: request may have already finished, so not necessarily an error

	// Accept notification
	UE::ModelContextProtocol::Private::CompleteWithResponseCode(OnComplete, EHttpServerResponseCodes::Accepted);
	return true;
}

bool FModelContextProtocolServer::ProcessListToolsJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete)
{
	FJsonDomBuilder::FArray ToolsArray;

	const IModelContextProtocolModule& ModelContextProtocolModule = IModelContextProtocolModule::GetChecked();
	for (const TSharedRef<IModelContextProtocolTool>& Tool : ModelContextProtocolModule.GetTools())
	{
		FJsonDomBuilder::FObject ToolObject;
		ToolObject.Set(TEXT("name"), Tool->GetName());
		if (const FString Description = Tool->GetDescription(); !Description.IsEmpty())
		{
			ToolObject.Set(TEXT("description"), Description);
		}
		if (const TSharedPtr<FJsonObject> InputJsonSchemaObject = Tool->GetInputJsonSchema())
		{
			ToolObject.Set(TEXT("inputSchema"), InputJsonSchemaObject);
		}
		if (const TSharedPtr<FJsonObject> OutputJsonSchemaObject = Tool->GetOutputJsonSchema())
		{
			ToolObject.Set(TEXT("outputSchema"), OutputJsonSchemaObject);
		}
		ToolsArray.Add(ToolObject);
	}

	UE_LOGF(LogModelContextProtocol, Log, "Listing tools (%d)", ToolsArray.Num());

	TSharedRef<FJsonValueArray> AllToolsJsonValue = ToolsArray.AsJsonValue();
	const TArray<TSharedPtr<FJsonValue>>& AllTools = AllToolsJsonValue->AsArray();
	UE::ModelContextProtocol::Private::FPaginationResult PaginationResult;
	FString PaginationError;
	if (!UE::ModelContextProtocol::Private::ApplyPagination(AllTools, Params, PaginationResult, PaginationError))
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidParams, PaginationError);
		return true;
	}

	FJsonDomBuilder::FObject ResultObject;
	ResultObject.Set(TEXT("tools"), MakeShared<FJsonValueArray>(MoveTemp(PaginationResult.Items)));
	if (PaginationResult.NextCursor.IsSet())
	{
		ResultObject.Set(TEXT("nextCursor"), PaginationResult.NextCursor.GetValue());
	}

	UE::ModelContextProtocol::Private::CompleteWithResult(OnComplete, RequestId, ResultObject);
	return true;
}

bool FModelContextProtocolServer::ProcessToolCallJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete)
{
	if (!Params.IsValid())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidParams, TEXT("Expected params for call to tools/call"));
		return true;
	}

	FString ToolName;
	if (!Params->TryGetStringField(TEXT("name"), ToolName) || ToolName.IsEmpty())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidParams, TEXT("Expected non-empty 'name' param for call to tools/call"));
		return true;
	}
	if (!RequestId.IsValid())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidParams, TEXT("Invalid request id"));
		return true;
	}

	TSharedPtr<FJsonObject> ToolArguments;
	if (Params->HasTypedField<EJson::Object>(TEXT("arguments")))
	{
		ToolArguments = Params->GetObjectField(TEXT("arguments"));
	}

	const IModelContextProtocolModule& ModelContextProtocolModule = IModelContextProtocolModule::GetChecked();
	TSharedPtr<IModelContextProtocolTool> Tool = ModelContextProtocolModule.FindTool(ToolName);
	if (!Tool.IsValid())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidParams, FString::Printf(TEXT("Unknown tool: %s"), *ToolName));
		return true;
	}

	UE_LOGF(LogModelContextProtocol, Log, "Running tool: '%ls'", *ToolName);
	FString PrettyJson;
	if (UE_LOG_ACTIVE(LogModelContextProtocol, VeryVerbose) && ToolArguments.IsValid() && FJsonSerializer::Serialize(ToolArguments.ToSharedRef(), TJsonWriterFactory<>::Create(&PrettyJson)))
	{
		UE_LOGF(LogModelContextProtocol, VeryVerbose, "'%ls' parameters:\n%ls",  *ToolName, *PrettyJson);
	}

	FModelContextProtocolToolRequestId ToolRequestId(RequestId.ToSharedRef());

	TSharedPtr<FJsonValue> ProgressToken;
	const TSharedPtr<FJsonObject>* MetaParams = nullptr;
	if (Params->TryGetObjectField(TEXT("_meta"), MetaParams))
	{
		if (MetaParams && MetaParams->IsValid())
		{
			ProgressToken = (*MetaParams)->TryGetField(TEXT("progressToken"));
		}
	}

	// Only open a streamed text/event-stream response when the caller requested progress
	// notifications (a progressToken). The empty first write exists solely to open the SSE stream so
	// progress events can be pushed while the tool runs; an ordinary tool call has no progress events,
	// so streaming its result as a second write over a keep-alive connection with no Content-Length
	// (and no chunked-transfer path in the engine HTTP server) leaves HTTP/1.1 clients unable to frame
	// the body. Those calls instead reply with a single application/json response below.
	const bool bUseStreaming = ProgressToken.IsValid();

	FModelContextProtocolToolContext& Context = Session->ActiveRequests.Add(ToolRequestId);
	Context.Tool = Tool;
	Context.ProgressToken = ProgressToken;
	Context.LastProgressSeconds = FPlatformTime::Seconds();

	const FString SessionId = Session->ID;

	if (bUseStreaming)
	{
		// Saving the callback for re-use with SSE event streaming.
		Context.EventStreamWrite = OnComplete;

		TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(FString(TEXT("")), UE::ModelContextProtocol::ContentTypeEventStream);
		Response->Headers.Add(TEXT("Connection"), { TEXT("keep-alive") });
		Response->Headers.Add(TEXT("Cache-Control"), { TEXT("no-cache") });
		Response->Headers.Add(UE::ModelContextProtocol::Private::McpSessionIdHeader, { SessionId });
		EnumAddFlags(Response->Flags, EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::HasAdditionalWrites);

		OnComplete(MoveTemp(Response));
	}

	TWeakPtr<bool> WeakAlive = AliveGuard;
	const double ToolCallStartTime = FPlatformTime::Seconds();
	// Hop to the game thread only when needed; see `IModelContextProtocolTool::FResultCallback` for the contract. The synchronous game-thread path avoids a tick of latency.
	auto HandleToolResult = [this, WeakAlive, RequestId, SessionId, ToolRequestId, ToolName, ToolArguments, ToolCallStartTime, OnComplete, bUseStreaming](const FModelContextProtocolToolResult& Result)
	{
		checkf(IsInGameThread(), TEXT("MCP tool result handler must run on the game thread; see IModelContextProtocolTool::FResultCallback."));

		if (!WeakAlive.IsValid())
		{
			return;
		}

		FJsonDomBuilder::FObject ResponseObject;
		ResponseObject.Set(TEXT("jsonrpc"), UE::ModelContextProtocol::JsonRpcVersion);
		ResponseObject.Set(TEXT("id"), RequestId);

		TSharedPtr<FModelContextProtocolSession> Session = FindSession(SessionId);

		if (!Session.IsValid() || !Session->ActiveRequests.Contains(ToolRequestId))
		{
			// Request was cancelled or session was destroyed — silently discard per MCP spec.
			// Skip analytics too: a cancelled request should not report a Success/Error outcome.
			return;
		}

		const double ToolCallDuration = FPlatformTime::Seconds() - ToolCallStartTime;
		const bool bSuccess = UE::ModelContextProtocol::Analytics::IsToolResultSuccess(Result.JsonObject);
		UE::ModelContextProtocol::Analytics::RecordToolCallEvent(SessionId, ToolName, ToolCallDuration, bSuccess);

		// Tool-search mode routes every actual toolset invocation through `call_tool`; without a second
		// emission every routed call collapses onto the empty-prefix toolset hash and per-toolset
		// analytics for the real targets stay invisible. Peek the call's arguments and emit one
		// additional `ToolCall` event for the resolved inner tool. The outer `call_tool` event above
		// remains so total dispatcher volume stays queryable. Self-dispatch (`call_tool` targeting
		// itself top-level) is filtered out so the rejection does not double-count under `call_tool`.
		if (ToolName.Equals(TEXT("call_tool"), ESearchCase::IgnoreCase) && ToolArguments.IsValid())
		{
			FString InnerToolsetName;
			ToolArguments->TryGetStringField(TEXT("toolset_name"), InnerToolsetName);
			FString InnerToolName;
			ToolArguments->TryGetStringField(TEXT("tool_name"), InnerToolName);
			const bool bIsSelfDispatch = InnerToolsetName.IsEmpty() && InnerToolName.Equals(TEXT("call_tool"), ESearchCase::IgnoreCase);
			if (!InnerToolName.IsEmpty() && !bIsSelfDispatch)
			{
				const FString ResolvedFullName = InnerToolsetName.IsEmpty()
					? InnerToolName
					: (InnerToolsetName + TEXT(".") + InnerToolName);
				UE::ModelContextProtocol::Analytics::RecordToolCallEvent(SessionId, ResolvedFullName, ToolCallDuration, bSuccess);
			}
		}

		FString PrettyResultJson;
		if (UE_LOG_ACTIVE(LogModelContextProtocol, VeryVerbose) && Result.JsonObject.IsValid()
			&& FJsonSerializer::Serialize(Result.JsonObject.ToSharedRef(), TJsonWriterFactory<>::Create(&PrettyResultJson)))
		{
			UE_LOGF(LogModelContextProtocol, VeryVerbose, "'%ls' result:\n%ls", *ToolName, *PrettyResultJson);
		}

		TSharedPtr<FJsonValue> ResultValue = MakeShared<FJsonValueObject>(Result.JsonObject);
		ResponseObject.Set(TEXT("result"), ResultValue);
		Session->ActiveRequests.Remove(ToolRequestId);

		if (!bUseStreaming)
		{
			// Ordinary (non-progress) tool call: reply with a single application/json response.
			// FHttpServerResponse::Create frames it with a Content-Length, so HTTP/1.1 clients can
			// detect end-of-message without a keep-alive stall, and the connection is never parked
			// awaiting a second write. This is the same single-write path used by initialize, ping,
			// tools/list, and the error responses above.
			UE::ModelContextProtocol::Private::CompleteWithResult(OnComplete, RequestId, ResultValue, &SessionId);
			return;
		}

		// Progress-streaming call: deliver the result as the final SSE write.
		FUtf8String ResponseStr;
		TSharedRef<TJsonWriter<UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR>>> JsonWriter = TJsonWriterFactory<UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR>>::Create(&ResponseStr);
		FJsonSerializer::Serialize(ResponseObject.AsJsonObject(), JsonWriter);

		TUniquePtr<FHttpServerResponse> ServerResponse = FHttpServerResponse::Create(UE::ModelContextProtocol::Private::FormatSSEMessage(ResponseStr), UE::ModelContextProtocol::ContentTypeEventStream);
		// `CloseAfterWrite` instructs the HTTPServer to tear down the connection after this final
		// SSE write completes. Required because the response carries no Content-Length and the
		// engine has no chunked-transfer path: HTTP/1.1 clients (libcurl, browser fetch) cannot
		// detect end-of-message on the unframed body and would otherwise stall ~15s waiting for
		// the keep-alive timeout to close the socket.
		EnumAddFlags(ServerResponse->Flags, EHttpServerResponseFlags::MultipleWriteStream | EHttpServerResponseFlags::SkipHeaderWrite | EHttpServerResponseFlags::CloseAfterWrite);

		OnComplete(MoveTemp(ServerResponse));
	};

	Tool->RunAsync(ToolRequestId, ToolArguments,
		[HandleToolResult = MoveTemp(HandleToolResult)](const FModelContextProtocolToolResult& Result)
		{
			if (IsInGameThread())
			{
				HandleToolResult(Result);
				return;
			}

			AsyncTask(ENamedThreads::GameThread, [HandleToolResult, Result]()
			{
				HandleToolResult(Result);
			});
		});

	return true;
}

bool FModelContextProtocolServer::ProcessListResourcesJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete)
{
	const IModelContextProtocolModule& ModelContextProtocolModule = IModelContextProtocolModule::GetChecked();

	LastResourceDescriptorList.Reset();
	for (const TSharedRef<IModelContextProtocolResourceProvider>& ResourceProvider : ModelContextProtocolModule.GetResourceProviders())
	{
		ResourceProvider->ListResources(LastResourceDescriptorList);
	}

	UE_LOGF(LogModelContextProtocol, Log, "Listing resources (%d)", LastResourceDescriptorList.Num());

	TSharedRef<FJsonValueArray> AllResourcesJsonValue = LastResourceDescriptorList.GetJsonArray();
	const TArray<TSharedPtr<FJsonValue>>& AllResources = AllResourcesJsonValue->AsArray();
	UE::ModelContextProtocol::Private::FPaginationResult PaginationResult;
	FString PaginationError;
	if (!UE::ModelContextProtocol::Private::ApplyPagination(AllResources, Params, PaginationResult, PaginationError))
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidParams, PaginationError);
		LastResourceDescriptorList.ReleaseJsonArray();
		return true;
	}

	FJsonDomBuilder::FObject ResultObject;
	ResultObject.Set(TEXT("resources"), MakeShared<FJsonValueArray>(MoveTemp(PaginationResult.Items)));
	if (PaginationResult.NextCursor.IsSet())
	{
		ResultObject.Set(TEXT("nextCursor"), PaginationResult.NextCursor.GetValue());
	}

	UE::ModelContextProtocol::Private::CompleteWithResult(OnComplete, RequestId, ResultObject);

	LastResourceDescriptorList.ReleaseJsonArray();

	return true;
}

bool FModelContextProtocolServer::ProcessReadResourceJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete)
{
	if (!Params.IsValid())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidParams, TEXT("Expected params for call to resources/read"));
		return true;
	}

	FString ResourceUri;
	if (!Params->TryGetStringField(TEXT("uri"), ResourceUri))
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InvalidParams, TEXT("Expected uri param for call to resources/read"));
		return true;
	}

	// Look up which resource provider can serve this resource. Check the cache first (populated by resources/list),
	// then fall back to querying all providers directly, since clients may call resources/read without prior resources/list.
	TSharedPtr<const IModelContextProtocolResourceProvider> ResourceProvider = LastResourceDescriptorList.FindResourceProvider(ResourceUri);
	if (!ResourceProvider.IsValid())
	{
		const IModelContextProtocolModule& ModelContextProtocolModule = IModelContextProtocolModule::GetChecked();
		for (const TSharedRef<IModelContextProtocolResourceProvider>& Provider : ModelContextProtocolModule.GetResourceProviders())
		{
			FModelContextProtocolResourceDescriptorList ProviderResources;
			Provider->ListResources(ProviderResources);
			if (ProviderResources.FindResourceProvider(ResourceUri).IsValid())
			{
				ResourceProvider = Provider;
				break;
			}
		}
	}

	if (!ResourceProvider.IsValid())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::ResourceNotFound, FString::Printf(TEXT("Resource not found: %s"), *ResourceUri));
		return true;
	}

	TValueOrError<FModelContextProtocolResource, FString> ResourceOrError = ResourceProvider->ReadResource(ResourceUri);
	if (ResourceOrError.HasError())
	{
		UE::ModelContextProtocol::Private::CompleteWithError(OnComplete, RequestId, UE::ModelContextProtocol::Private::EJsonRpcErrorCode::InternalError, FString::Printf(TEXT("Error reading resource: %s"), *ResourceOrError.GetError()));
		return true;
	}

	FJsonDomBuilder::FObject ResultObject;
	ResultObject.Set(TEXT("contents"), FJsonDomBuilder::FArray().Add(ResourceOrError.GetValue().GetJsonObject()));

	UE::ModelContextProtocol::Private::CompleteWithResult(OnComplete, RequestId, ResultObject);
	return true;
}

void FModelContextProtocolServer::Tick(float DeltaTime)
{
	if (bToolsListChangedBroadcastScheduled)
	{
		bToolsListChangedBroadcastScheduled = false;

		// Delivery is gated on at least one in-flight `tools/call` per session: this server returns
		// `BadMethod` on GET (see `ProcessGetRequest`), so there is no persistent server-push channel.
		// Quiet sessions miss this notification until they initiate the next request and open a new SSE stream.
		for (const TSharedPtr<FModelContextProtocolSession>& Session : Sessions)
		{
			if (!Session.IsValid() || Session->Status != EModelContextProtocolSessionStatus::Initialized)
			{
				continue;
			}

			for (auto& [RequestId, Context] : Session->ActiveRequests)
			{
				if (Context.EventStreamWrite.IsSet())
				{
					Context.EventStreamWrite(UE::ModelContextProtocol::Private::CreateJsonRpcNotificationResponse(TEXT("notifications/tools/list_changed")));
					// One write per session: every active SSE stream within a session belongs to the same MCP client.
					break;
				}
			}
		}

		UE_LOGF(LogModelContextProtocol, Log, "Broadcast notifications/tools/list_changed to active sessions");
	}

	if (UE::ModelContextProtocol::ProgressIntervalSeconds > 0.0f)
	{
		const double NowSeconds = FPlatformTime::Seconds();

		for (TSharedPtr<FModelContextProtocolSession>& Session : Sessions)
		{
			if (Session.IsValid())
			{
				for (TPair<FModelContextProtocolToolRequestId, FModelContextProtocolToolContext>& ToolRequest : Session->ActiveRequests)
				{
					FModelContextProtocolToolContext& Context = ToolRequest.Value;

					if (Context.ProgressToken.IsValid() && Context.EventStreamWrite.IsSet())
					{
						if ((NowSeconds - Context.LastProgressSeconds) >= UE::ModelContextProtocol::ProgressIntervalSeconds)
						{
							// Not sending a total progress value, so this is simply a heartbeat as the total duration is unknown
							++Context.LastProgressValue;

							UE::ModelContextProtocol::Private::SendProgressUpdate(Context.EventStreamWrite, Context.ProgressToken.ToSharedRef(), Context.LastProgressValue);

							Context.LastProgressSeconds = NowSeconds;
						}
					}
				}
			}
		}
	}
}

bool FModelContextProtocolServer::ProcessGetRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	if (!UE::ModelContextProtocol::Private::ValidateOriginHeader(Request, OnComplete))
	{
		return true;
	}

	// We do not currently support sse on a separate endpoint
	UE::ModelContextProtocol::Private::CompleteWithResponseCode(OnComplete, EHttpServerResponseCodes::BadMethod);
	return true;
}

bool FModelContextProtocolServer::ProcessDeleteRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	if (!UE::ModelContextProtocol::Private::ValidateOriginHeader(Request, OnComplete))
	{
		return true;
	}

	FString SessionId;
	if (UE::ModelContextProtocol::Private::GetSessionIdFromRequestHeaders(Request, SessionId))
	{
		// Snapshot whether the session reached Initialized before removal so we emit SessionEnd only
		// for sessions that actually had a paired SessionStart. Prevents phantom events for
		// client-supplied garbage session ids or sessions that dropped mid-initialize.
		const TSharedPtr<FModelContextProtocolSession> Session = FindSession(SessionId);
		const bool bWasInitialized = Session.IsValid() && Session->Status == EModelContextProtocolSessionStatus::Initialized;

		const int32 RemovedCount = Sessions.RemoveAll([&SessionId](const TSharedPtr<FModelContextProtocolSession>& InSession)
			{
				return InSession.IsValid() && (InSession->ID == SessionId);
			});

		if (RemovedCount > 0)
		{
			if (bWasInitialized)
			{
				UE::ModelContextProtocol::Analytics::RecordSessionEndEvent(SessionId);
			}
			UE::ModelContextProtocol::Private::CompleteWithResponseCode(OnComplete, EHttpServerResponseCodes::Accepted);
		}
		else
		{
			UE::ModelContextProtocol::Private::CompleteWithResponseCode(OnComplete, EHttpServerResponseCodes::BadRequest);
		}
	}
	else
	{
		UE::ModelContextProtocol::Private::CompleteWithResponseCode(OnComplete, EHttpServerResponseCodes::BadRequest);
	}

	return true;
}
