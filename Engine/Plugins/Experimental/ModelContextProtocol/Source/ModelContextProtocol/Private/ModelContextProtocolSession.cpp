// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolSession.h"
#include "Dom/JsonValue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelContextProtocolSession)

bool FModelContextProtocolToolRequestId::operator==(const FModelContextProtocolToolRequestId& RHS) const
{
	return IsValid() && RHS.IsValid() && FJsonValue::CompareEqual(*RequestId.Get(), *RHS.RequestId.Get());
}

uint32 GetTypeHash(const FModelContextProtocolToolRequestId& InRequestId)
{
	return InRequestId.RequestId.IsValid() ? GetTypeHash(InRequestId.RequestId->AsString()) : 0;
}
