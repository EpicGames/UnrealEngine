// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolResults.h"
#include "ModelContextProtocol.h"

#include "JsonObjectConverter.h"
#include "ModelContextProtocolResources.h"
#include "Misc/Base64.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelContextProtocolToolResults)

namespace UE::ModelContextProtocol::Private
{
	TSharedRef<FJsonObject> MakeResourceLinkContentObject(const FModelContextProtocolResourceDescriptor& ResourceDescriptor, EModelContextProtocolAudience Audience)
	{
		FJsonDomBuilder::FObject ResourceLinkContentObject;
		FJsonObject::Duplicate(ResourceDescriptor.GetJsonObject(), ResourceLinkContentObject.AsJsonObject());
		ResourceLinkContentObject.Set(TEXT("type"), TEXT("resource_link"));

		FJsonDomBuilder::FArray Audiences;
		if (EnumHasAnyFlags(Audience, EModelContextProtocolAudience::User))
		{
			Audiences.Add(FString(TEXT("user")));
		}
		if (EnumHasAnyFlags(Audience, EModelContextProtocolAudience::Assistant))
		{
			Audiences.Add(FString(TEXT("assistant")));
		}
		if (Audiences.Num() > 0)
		{
			ResourceLinkContentObject.Set(TEXT("annotations"), FJsonDomBuilder::FObject()
				.Set(TEXT("audience"), Audiences));
		}
		return ResourceLinkContentObject.AsJsonObject();
	}
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeImageResult(const FString& MimeType, TArrayView<uint8> Data, EModelContextProtocolAudience Audience)
{
	FString Base64EncodedData = FBase64::Encode(Data.GetData(), Data.NumBytes());
	return MakeImageResult(MimeType, MoveTemp(Base64EncodedData), Audience);
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeImageResult(const FString& MimeType, FString&& Base64EncodedData, EModelContextProtocolAudience Audience)
{
	FJsonDomBuilder::FObject ImageContentObject;
	ImageContentObject.Set(TEXT("type"), TEXT("image"));
	ImageContentObject.Set(TEXT("mimeType"), MimeType);
	ImageContentObject.Set(TEXT("data"), MoveTemp(Base64EncodedData));
	FJsonDomBuilder::FArray Audiences;
	if (EnumHasAnyFlags(Audience, EModelContextProtocolAudience::User))
	{
		Audiences.Add(FString(TEXT("user")));
	}
	if (EnumHasAnyFlags(Audience, EModelContextProtocolAudience::Assistant))
	{
		Audiences.Add(FString(TEXT("assistant")));
	}
	if (Audiences.Num() > 0)
	{
		ImageContentObject.Set(TEXT("annotations"), FJsonDomBuilder::FObject()
			.Set(TEXT("audience"), Audiences));
	}
	
	FJsonDomBuilder::FArray ContentArray;
	ContentArray.Add(ImageContentObject);

	FJsonDomBuilder::FObject ResultObject;
	ResultObject.Set(TEXT("content"), ContentArray);

	return ResultObject.AsJsonObject().ToSharedPtr();
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeAudioResult(const FString& MimeType, TArrayView<uint8> Data)
{
	FString Base64EncodedData = FBase64::Encode(Data.GetData(), Data.NumBytes());
	return MakeAudioResult(MimeType, MoveTemp(Base64EncodedData));
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeAudioResult(const FString& MimeType, FString&& Base64EncodedData)
{
	FJsonDomBuilder::FObject ImageContentObject;
	ImageContentObject.Set(TEXT("type"), TEXT("audio"));
	ImageContentObject.Set(TEXT("mimeType"), MimeType);
	ImageContentObject.Set(TEXT("data"), MoveTemp(Base64EncodedData));
	
	FJsonDomBuilder::FArray ContentArray;
	ContentArray.Add(ImageContentObject);

	FJsonDomBuilder::FObject ResultObject;
	ResultObject.Set(TEXT("content"), ContentArray);

	return ResultObject.AsJsonObject().ToSharedPtr();
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeResourceLinkResult(const FModelContextProtocolResourceDescriptor& ResourceDescriptor, EModelContextProtocolAudience Audience)
{
	TSharedRef<FJsonObject> ResourceLinkContentObject = Private::MakeResourceLinkContentObject(ResourceDescriptor, Audience);
	
	FJsonDomBuilder::FArray ContentArray;
	ContentArray.Add(ResourceLinkContentObject);

	FJsonDomBuilder::FObject ResultObject;
	ResultObject.Set(TEXT("content"), ContentArray);

	return ResultObject.AsJsonObject().ToSharedPtr();
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeResourceLinksResult(TConstArrayView<FModelContextProtocolResourceDescriptor> ResourceDescriptors, EModelContextProtocolAudience Audience)
{
	FJsonDomBuilder::FArray ContentArray;

	for (const FModelContextProtocolResourceDescriptor& ResourceDescriptor : ResourceDescriptors)
	{
		TSharedRef<FJsonObject> ResourceLinkContentObject = Private::MakeResourceLinkContentObject(ResourceDescriptor, Audience);
		ContentArray.Add(ResourceLinkContentObject);
	}

	FJsonDomBuilder::FObject ResultObject;
	ResultObject.Set(TEXT("content"), ContentArray);

	return ResultObject.AsJsonObject().ToSharedPtr();
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeStructuredContentResult(const UStruct* StructDefinition, const void* Struct, const FJsonExportParams& Params)
{
	if (!ensure(StructDefinition) || !ensure(Struct))
	{
		return FModelContextProtocolToolResult();
	}
	
	if (StructDefinition == FModelContextProtocolToolResult::StaticStruct())
	{
		return *static_cast<const FModelContextProtocolToolResult*>(Struct);
	}
	
	TSharedRef<FJsonObject> StructuredContentObject = MakeShared<FJsonObject>();
	if (FJsonObjectConverter::UStructToJsonObject(StructDefinition, Struct, StructuredContentObject, Params.CheckFlags, Params.SkipFlags, Params.ExportCb, Params.ConversionFlags))
	{
		TSharedPtr<FJsonValue> StructuredContentValue = MakeShared<FJsonValueObject>(MoveTemp(StructuredContentObject));
		return MakeStructuredContentResult(StructuredContentValue);
	}
	
	return FModelContextProtocolToolResult();
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeStructuredContentResult(FProperty* Property, const void* Value, const FJsonExportParams& Params)
{
	// Return FModelContextProtocolToolResult's directly
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		if (StructProperty->Struct == FModelContextProtocolToolResult::StaticStruct())
		{
			return *static_cast<const FModelContextProtocolToolResult*>(Value);
		}
	}
	
	if (TSharedPtr<FJsonValue> StructuredContent = FJsonObjectConverter::UPropertyToJsonValue(Property, Value, Params.CheckFlags, Params.SkipFlags, Params.ExportCb, /*OuterProperty*/nullptr, Params.ConversionFlags))
	{
		return MakeStructuredContentResult(StructuredContent);
	}

	return FModelContextProtocolToolResult();
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeStructuredContentResult(TSharedPtr<FJsonValue> StructuredContent)
{
	if (!ensure(StructuredContent.IsValid()))
	{
		return MakeErrorResult(TEXT("Expected structured content result"));
	}

	// {"result":} object wrapper for POD results?
	if (UE::ModelContextProtocol::bWrapPODResultsInObject && StructuredContent->Type != EJson::Object)
	{
		StructuredContent = FJsonDomBuilder::FObject()
			.Set(UE::ModelContextProtocol::PODWrapperResultPropertyName, StructuredContent)
			.AsJsonValue();
	}
	
	// StructuredContent -> text part for backwards compatibility (as per the spec) 
	FString StructuredContentString;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&StructuredContentString);
	FJsonSerializer::Serialize(StructuredContent.ToSharedRef(), /*Identifier*/FString(), JsonWriter);
	
	FJsonDomBuilder::FArray ContentArray;
	ContentArray.Add(MakeTextContentObject(MoveTemp(StructuredContentString)));

	FJsonDomBuilder::FObject ResultObject;
	ResultObject.Set(TEXT("content"), ContentArray);
	ResultObject.Set(TEXT("structuredContent"), StructuredContent);
	
	return ResultObject.AsJsonObject().ToSharedPtr();
}
