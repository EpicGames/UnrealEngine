// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolServerAccessor.h"

#include "ModelContextProtocolServer.h"
#include "ModelContextProtocolSession.h"

#include "Dom/JsonValue.h"
#include "Templates/SharedPointer.h"

namespace UE::ModelContextProtocol
{
	void FServerAccessor::AddInitializedSessionWithEventStream(FModelContextProtocolServer& Server, const FHttpResultCallback& StreamWrite)
	{
		TSharedPtr<FModelContextProtocolSession> Session = MakeShared<FModelContextProtocolSession>();
		Session->Status = EModelContextProtocolSessionStatus::Initialized;
		FModelContextProtocolToolRequestId RequestId(MakeShared<FJsonValueString>(TEXT("test")));
		FModelContextProtocolToolContext& Context = Session->ActiveRequests.Add(RequestId);
		Context.EventStreamWrite = StreamWrite;
		Server.Sessions.Add(Session);
	}

	void FServerAccessor::Tick(FModelContextProtocolServer& Server, float DeltaTime)
	{
		Server.Tick(DeltaTime);
	}

	bool FServerAccessor::IsBroadcastScheduled(const FModelContextProtocolServer& Server)
	{
		return Server.bToolsListChangedBroadcastScheduled;
	}
}
