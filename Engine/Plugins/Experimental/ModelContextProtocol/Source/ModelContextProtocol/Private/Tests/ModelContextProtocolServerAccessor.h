// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HttpResultCallback.h"

class FModelContextProtocolServer;

namespace UE::ModelContextProtocol
{
	// Test-only accessor for `FModelContextProtocolServer` privates. Friended by the production class.
	// Implementations live in the production module so they can touch private state without DLL exports
	// leaking onto the production class itself.
	class FServerAccessor
	{
	public:
		// Inject one Initialized session that owns a single ActiveRequest whose EventStreamWrite is the given callback.
		static MODELCONTEXTPROTOCOL_API void AddInitializedSessionWithEventStream(FModelContextProtocolServer& Server, const FHttpResultCallback& StreamWrite);

		// Drive the server's per-frame tick once.
		static MODELCONTEXTPROTOCOL_API void Tick(FModelContextProtocolServer& Server, float DeltaTime);

		// Peek whether a `tools/list_changed` broadcast is currently scheduled but not yet drained.
		static MODELCONTEXTPROTOCOL_API bool IsBroadcastScheduled(const FModelContextProtocolServer& Server);
	};
}
