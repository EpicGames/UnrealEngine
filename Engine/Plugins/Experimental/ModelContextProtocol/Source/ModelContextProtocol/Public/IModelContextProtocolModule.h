// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocol.h"
#include "Modules/ModuleManager.h"

class FModelContextProtocolServer;
class IAnalyticsProviderET;
struct FAnalyticsEventAttribute;
struct IModelContextProtocolResourceProvider;
struct IModelContextProtocolTool;

#define UE_API MODELCONTEXTPROTOCOL_API


class IModelContextProtocolModule : public IModuleInterface
{
public:

	[[nodiscard]] UE_API static IModelContextProtocolModule* Get();
	[[nodiscard]] UE_API static IModelContextProtocolModule& GetChecked();

	/** Returns current list of registered tools being served */
	virtual const TArray<TSharedRef<IModelContextProtocolTool>>& GetTools() const = 0;

	/**
	 * Find tool by name (case-insensitive)
	 * @see IModelContextProtocolTool::GetName
	 */
	[[nodiscard]] virtual TSharedPtr<IModelContextProtocolTool> FindTool(const FString& ToolName) const = 0;

	/**
	 * Add tool to server, ensuring no tool is already registered by the same name (case-insensitive).
	 * Note: Tool providers *should* listen for OnRefreshTools to re-add tools. 
	 * @see IModelContextProtocolTool::GetName
	 * @return true if the tool was successfully registered, false if there is already a tool by this name being served.
	 */
	virtual bool AddTool(const TSharedRef<IModelContextProtocolTool>& Tool) = 0;

	/**
	 * Remove Tool from server (by pointer)
	 * @param Tool	The tool to remove
	 * @return true if Tool was found and removed from the collection, false if Tool wasn't found in the collection.
	 */	
	virtual bool RemoveTool(const TSharedRef<IModelContextProtocolTool>& Tool) = 0;

	DECLARE_MULTICAST_DELEGATE(FOnRefreshTools);
	
	/**
	 * @return	A multicast delegate which is executed whenever the ModelContextProtocol.RefreshTools console command is run to release all current
	 *			tools and rebuild tool registrations.
	 */
	virtual FOnRefreshTools& OnRefreshTools() = 0;

	/**
	 * Called by ModelContextProtocol.RefreshTools console command. Releases all currently registered tools and broadcasts OnRefreshTools for tool
	 * providers to re-add tools.
	 */
	virtual void RefreshTools() = 0;

	/** Returns the MCP server instance, or nullptr if not created. */
	[[nodiscard]] virtual FModelContextProtocolServer* GetServer() = 0;

	/** Creates and starts the MCP HTTP server on the specified port and URL path. */
	virtual void StartServer(uint32 Port = UE::ModelContextProtocol::DefaultServerPort, const FString& UrlPath = UE::ModelContextProtocol::DefaultServerUrlPath) = 0;

	/** Stops and destroys the MCP HTTP server. */
	virtual void StopServer() = 0;

	/**
	 * Sets the analytics provider used to record MCP telemetry events.
	 * A strong reference to the provider is held until a new or null provider is set.
	 * @param AnalyticsProvider The provider to use, or nullptr to clear.
	 */
	virtual void SetAnalyticsProvider(const TSharedPtr<IAnalyticsProviderET>& AnalyticsProvider) = 0;

	/**
	 * @return The currently installed analytics provider, or nullptr if none is set.
	 */
	[[nodiscard]] virtual TSharedPtr<IAnalyticsProviderET> GetAnalyticsProvider() const = 0;

	/**
	 * Sets the namespace prepended (with "." separator) to all MCP analytics event names.
	 * Default is "ModelContextProtocol". Pass an empty string to emit events without a prefix.
	 */
	virtual void SetAnalyticsEventNamespace(const FString& Namespace) = 0;

	/**
	 * @return A copy of the current analytics event namespace.
	 *         Returned by value so callers don't hold a reference across racing Set calls.
	 */
	[[nodiscard]] virtual FString GetAnalyticsEventNamespace() const = 0;

	/**
	 * Records an MCP analytics event via the currently installed provider. No-op if no provider is set.
	 * The event name is prepended with the namespace returned by GetAnalyticsEventNamespace.
	 */
	virtual void RecordAnalyticsEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) = 0;

	/**
	 * Returns current list of registered resource providers.
	 * @see AddResourceProvider, FindResourceProvider
	 */
	virtual const TArray<TSharedRef<IModelContextProtocolResourceProvider>>& GetResourceProviders() const = 0;

	/**
	 * Add a resource provider.
	 * Subsequent MCP resources/list commands will call upon this provider to list resources in response.
	 * @param ResourceProvider The provider to register.
	 */
	virtual void AddResourceProvider(const TSharedRef<IModelContextProtocolResourceProvider>& ResourceProvider) = 0;

	/**
	 * Remove / deregister resource provider.
	 * @param ResourceProvider The provider to remove / de-register.
	 * @return true if ResourceProvider was found and deregistered, false if ResourceProvider wasn't registered.
	 */	
	virtual bool RemoveResourceProvider(const TSharedRef<IModelContextProtocolResourceProvider>& ResourceProvider) = 0;
};

#undef UE_API
