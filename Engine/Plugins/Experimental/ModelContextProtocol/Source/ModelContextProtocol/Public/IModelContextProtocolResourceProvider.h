// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"

#define UE_API MODELCONTEXTPROTOCOL_API

struct FModelContextProtocolResource;
struct FModelContextProtocolResourceDescriptorList;

/**
 * Abstract interface for exposing MCP (Anthropic's Model Context Protocol) resources.
 * 
 * Resource providers must be registered via IModelContextProtocolModule::GetChecked().AddResourceProvider(MyProvider)
 *
 * @see https://modelcontextprotocol.io/specification/2025-06-18/server/resources
 */
struct IModelContextProtocolResourceProvider : TSharedFromThis<IModelContextProtocolResourceProvider>
{
	UE_API virtual ~IModelContextProtocolResourceProvider();
	
	/**
	 * List / describe all resources this provider can currently provide.
	 * 
	 * Any listed resource Uri's which are then requested for read, will be routed back to this provider for reading via this provider's
	 * ReadResource implementation.
	 */  
	virtual void ListResources(FModelContextProtocolResourceDescriptorList& OutResourceDescriptors) const = 0;

	/** Read the contents of a previously listed resource */
	virtual TValueOrError<FModelContextProtocolResource, FString> ReadResource(const FString& Uri) const = 0;
};

#undef UE_API
