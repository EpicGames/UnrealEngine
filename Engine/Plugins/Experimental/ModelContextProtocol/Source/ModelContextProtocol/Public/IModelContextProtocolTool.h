// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocolToolResults.h"

#include "Dom/JsonObject.h"
#include "JsonDomBuilder.h"
#include "Templates/SharedPointer.h"

#define UE_API MODELCONTEXTPROTOCOL_API

class FReferenceCollector;
struct FModelContextProtocolToolRequestId;

/**
 * Abstract interface for a describable MCP (Anthropic's Model Context Protocol) tool.
 * 
 * Tools can be registered for use via IModelContextProtocolModule::GetChecked().AddTool and must implement either Run or RunAsync for execution.
 *
 * @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools
 */
struct IModelContextProtocolTool : TSharedFromThis<IModelContextProtocolTool>
{
	/**
	 * Tool result object or error.
	 * Result json object must be complete MCP result object with either content array or structuredContent
	 * @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools#tool-result
	 *
	 * Threading: tool implementations may invoke this callback from any thread. The MCP server hops to the game thread before consuming the result, so per-tool work that completes off the game thread does not need to marshal back itself.
	 *
	 * The callback must be invoked exactly once per tool call. Subsequent invocations after the request has been served (or cancelled) are silently dropped.
	 */
	typedef TFunction<void(const FModelContextProtocolToolResult& Result)> FResultCallback;

	virtual ~IModelContextProtocolTool() = default;

	/** Returns the identifier name for this tool e.g: select_object */ 
	virtual FString GetName() const = 0;

	/** Returns a description of what this tool does when executed, and when it should be used */
	virtual FString GetDescription() const = 0;

	/**
	 * Returns the JSON Schema definition which the input parameters to this tool must adhere to.
	 * Per the MCP specification, inputSchema is required and must be a JSON Schema object
	 * with "type": "object". The default returns {"type": "object"} (accepts any parameters).
	 * @return A valid JSON Schema definition.
	 * @see https://json-schema.org
	 * @see FJsonSchemaGenerator::UStructToJsonSchemaObject
	 */
	virtual TSharedPtr<FJsonObject> GetInputJsonSchema() const
	{
		FJsonDomBuilder::FObject Schema;
		Schema.Set(TEXT("type"), TEXT("object"));
		return Schema.AsJsonObject().ToSharedPtr();
	}

	/**
	 * Returns an optional JSON Schema definition which the results of this tool must adhere to.
	 * @return A valid JSON Schema definition or invalid shared pointer for undefined output.  
	 * @see https://json-schema.org
	 * @see FJsonSchemaGenerator::UStructToJsonSchemaObject
	 */
	virtual TSharedPtr<FJsonObject> GetOutputJsonSchema() const { return {}; }

	/**
	 * Executes this tool, performing the described operations, using Params as input, immediately returning a result. 
	 * @param Params	Input parameters. If GetInputJsonSchema returns a valid schema object, these parameters *should* adhere to said schema.
	 *					Note: Schema adherence by MCP clients is assumed, not strictly checked internally.
	 * @return 
	 * @see RunAsync
	 */
	virtual FModelContextProtocolToolResult Run(const TSharedPtr<FJsonObject>& Params)
	{
		ensureMsgf(false, TEXT("IModelContextProtocolTool must implement either Run or RunAsync"));
		return UE::ModelContextProtocol::MakeErrorResult(TEXT("IModelContextProtocolTool must implement either Run or RunAsync"));
	};

	/**
	 * Executes this tool asynchronously, performing the described operations over time, using Params as input. Once operations are complete,
	 * OnComplete is called with any results.
	 * Note: Calls synchronous Run by default, immediately executing OnComplete with the synchronous result. 
	 * @param RequestId The request id from the client request for identifying this particular tool call.
	 * @param Params	Input parameters. If GetInputJsonSchema returns a valid schema object, these parameters *should* adhere to said schema.
	 *					Note: Schema adherence by MCP clients is assumed, not strictly checked internally. 
	 * @param OnComplete	Once the operation is complete, implementations must call OnComplete, passing in their JSON formatted tool results.
	 *						Implementations returning a valid schema from GetOutputJsonSchema *MUST* ensure results adhere to this schema.
	 * @see Run
	 */
	virtual void RunAsync(const FModelContextProtocolToolRequestId& RequestId, const TSharedPtr<FJsonObject>& Params, const FResultCallback& OnComplete)
	{
		const FModelContextProtocolToolResult Result = Run(Params);
		OnComplete(Result);
	}

	virtual void CancelAsync(const FModelContextProtocolToolRequestId& RequestId) {}

	/** Called by FModelContextProtocolToolCollection to allow tools to report UObject dependencies for reference tracking */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) {}
};

#undef UE_API
