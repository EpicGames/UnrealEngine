// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

struct IModelContextProtocolResourceProvider;
struct IModelContextProtocolTool;

#define UE_API MODELCONTEXTPROTOCOL_API

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogModelContextProtocol, Log, All);

namespace UE::ModelContextProtocol
{
	static constexpr const TCHAR* JsonRpcVersion = TEXT("2.0");

	/** The latest protocol version supported by this server. */
	static constexpr const TCHAR* ProtocolVersion = TEXT("2025-11-25");

	/** All protocol versions this server supports, ordered newest-first. */
	inline const TArray<FString>& GetSupportedProtocolVersions()
	{
		static const TArray<FString> Versions = {
			TEXT("2025-11-25"),
			TEXT("2025-06-18"),
			TEXT("2024-11-05"),
		};
		return Versions;
	}

	/** Returns the negotiated protocol version: the client's version if supported, otherwise the server's latest. */
	inline FString NegotiateProtocolVersion(const FString& ClientRequestedVersion)
	{
		for (const FString& Version : GetSupportedProtocolVersions())
		{
			if (Version == ClientRequestedVersion)
			{
				return ClientRequestedVersion;
			}
		}
		return ProtocolVersion;
	}

	static constexpr uint32 DefaultServerPort = 8000;
	static constexpr const TCHAR* DefaultServerUrlPath = TEXT("/mcp");
	static constexpr const TCHAR* DefaultServerName = TEXT("unreal-mcp");

	/** Tool name validation result per MCP spec (2025-11-25). */
	enum class EToolNameValidation : uint8
	{
		Valid,
		Empty,
		ExceedsMaxLength,
		InvalidCharacters
	};

	/**
	 * Validates a tool name per MCP spec (2025-11-25) tool name rules.
	 * Names SHOULD be 1-128 chars using only A-Z, a-z, 0-9, underscore, hyphen, and dot.
	 * @see https://modelcontextprotocol.io/specification/2025-11-25/server/tools#tool-names
	 */
	UE_API EToolNameValidation ValidateToolName(const FString& ToolName);

	static constexpr const TCHAR* PODWrapperResultPropertyName = TEXT("result");

	static constexpr const TCHAR* ContentTypeEventStream = TEXT("text/event-stream");

	// CVars
	UE_API extern bool bWrapPODResultsInObject;
	UE_API extern bool bAudioResultOggFormat;
	UE_API extern float ProgressIntervalSeconds;
	UE_API extern int32 PaginationPageSize;
}

#undef UE_API
