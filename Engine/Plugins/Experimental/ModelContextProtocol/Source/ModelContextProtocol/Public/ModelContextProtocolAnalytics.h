// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define UE_API MODELCONTEXTPROTOCOL_API

class FJsonObject;
class IAnalyticsProviderET;

namespace UE::ModelContextProtocol::Analytics
{
	/**
	 * Resolves the success/error outcome of a tool call from its result JSON.
	 * Per the MCP spec, "isError" is optional and defaults to false.
	 * - Null JsonObject: Error (malformed or missing result).
	 * - No "isError" field: Success (MCP default).
	 * - "isError" is a bool: follow it.
	 * - "isError" is present but not a bool: Error (malformed).
	 */
	UE_API bool IsToolResultSuccess(const TSharedPtr<FJsonObject>& ResultJson);

	/**
	 * Returns a lowercase-hex Blake3 hash of an MCP tool or toolset identifier.
	 * Input is encoded as UTF-8 before hashing so the result is stable across platforms.
	 * Used to anonymize tool names in analytics payloads: Epic resolves known hashes against
	 * its own lookup table; licensee-private tool names stay opaque.
	 */
	UE_API FString HashToolIdentifier(const FString& Identifier);

	/**
	 * Extracts the toolset portion of an MCP tool name by splitting at the last dot.
	 * Matches the semantics of `UE::ToolsetRegistry::FToolDescriptor::FromString`. Returns an empty string for top-level tools (no dot present).
	 */
	UE_API FString ParseToolsetName(const FString& FullToolName);

	/** Records a tool call completion event. */
	UE_API void RecordToolCallEvent(const FString& SessionId, const FString& ToolName, double DurationSeconds, bool bSuccess);

	/** Records a session start event (called on notifications/initialized). */
	UE_API void RecordSessionStartEvent(const FString& SessionId, const FString& NegotiatedProtocolVersion);

	/** Records a session end event (called on DELETE or server shutdown). */
	UE_API void RecordSessionEndEvent(const FString& SessionId);
}

#undef UE_API
