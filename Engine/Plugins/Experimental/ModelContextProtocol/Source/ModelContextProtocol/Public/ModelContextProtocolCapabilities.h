// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocolCapabilities.generated.h"

#define UE_API MODELCONTEXTPROTOCOL_API

USTRUCT()
struct FModelContextProtocolRootsCapability
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<bool> ListChanged;
};

USTRUCT()
struct FModelContextProtocolSamplingCapability
{
	GENERATED_BODY()
};

USTRUCT()
struct FModelContextProtocolElicitationCapability
{
	GENERATED_BODY()
};

USTRUCT()
struct FModelContextProtocolClientCapabilities
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<FModelContextProtocolRootsCapability> Roots;

	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<FModelContextProtocolSamplingCapability> Sampling;

	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<FModelContextProtocolElicitationCapability> Elicitation;
};

USTRUCT()
struct FModelContextProtocolLoggingCapability
{
	GENERATED_BODY()
};

USTRUCT()
struct FModelContextProtocolPromptsCapability
{
	GENERATED_BODY()
	
	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<bool> ListChanged;
};

USTRUCT()
struct FModelContextProtocolResourcesCapability
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<bool> Subscribe;
	
	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<bool> ListChanged;
};

USTRUCT()
struct FModelContextProtocolToolsCapability
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<bool> ListChanged;
};

USTRUCT()
struct FModelContextProtocolServerCapabilities
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<FModelContextProtocolLoggingCapability> Logging; 
	
	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<FModelContextProtocolPromptsCapability> Prompts; 

	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<FModelContextProtocolResourcesCapability> Resources; 

	UPROPERTY(VisibleAnywhere, Category="Capabilities")
	TOptional<FModelContextProtocolToolsCapability> Tools; 
};

#undef UE_API
