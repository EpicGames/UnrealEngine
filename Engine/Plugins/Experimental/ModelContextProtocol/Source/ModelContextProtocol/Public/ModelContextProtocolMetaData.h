// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Misc/Optional.h"

#include "ModelContextProtocolMetaData.generated.h"

struct FJsonSchemaEditorMetadata;
struct FJsonSchemaPropertyFilter;

/** Cached editor-only property meta-data in a cookable form */
USTRUCT()
struct FModelContextProtocolPropertyMetaData
{
	GENERATED_BODY()

	/** Property description derived from its tooltip / native code comment */
	UPROPERTY()
	TOptional<FText> Description;

	/** Property's Meta ClampMin e.g: UPROPERTY(Meta = (ClampMin = "1")) */
	UPROPERTY()
	TOptional<double> ClampMin;

	/** Property's Meta ClampMax e.g: UPROPERTY(Meta = (ClampMax = "1000")) */
	UPROPERTY()
	TOptional<double> ClampMax;

	/** For function parameters, stores the default parameter value */
	UPROPERTY()
	TOptional<FString> DefaultValue;

	/** Original JSON type (EJson cast to uint8) of DefaultValue, used for lossless round-trip */
	UPROPERTY()
	TOptional<uint8> DefaultValueJsonType;
};

/** Cached editor-only function meta-data in a cookable form */
USTRUCT()
struct FModelContextProtocolFunctionMetaData
{
	GENERATED_BODY()

	/** Function description derived from its tooltip / native code comment */
	UPROPERTY()
	TOptional<FText> Description;

	/** Per-property meta data, including inner properties e.g: Person, Person.Name, Person.Age */
	UPROPERTY()
	TMap<FString, FModelContextProtocolPropertyMetaData> PropertyMetaData;

	/** The function's WorldContext meta-data value, if any */
	UPROPERTY()
	FString WorldContext;
};

