// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "JsonDomBuilder.h"
#include "Misc/Base64.h"
#include "Templates/SharedPointer.h"
#include "Templates/ValueOrError.h"

#define UE_API MODELCONTEXTPROTOCOL_API

struct IModelContextProtocolResourceProvider;

/**
 * Identifying meta-data for a potential resource to be provided by an IModelContextProtocolResourceProvider.
 */
struct FModelContextProtocolResourceDescriptor
{
	UE_API FModelContextProtocolResourceDescriptor(const FString& Uri, const TOptional<FString>& Name = {}, const TOptional<FString>& Title = {}, const TOptional<FString>& Description = {}, const TOptional<FString>& MimeType = {});
	UE_API FString GetUri() const;
	UE_API TSharedRef<FJsonObject> GetJsonObject() const;

protected:
	FJsonDomBuilder::FObject JsonResourceObject;
};

/**
 * A string or binary blob resource, provided by an IModelContextProtocolResourceProvider
 */
struct FModelContextProtocolResource : public FModelContextProtocolResourceDescriptor
{
	UE_API FModelContextProtocolResource(const FString& Uri, FString&& TextContent, const TOptional<FString>& Name = {}, const TOptional<FString>& Title = {}, const TOptional<FString>& MimeType = {});
	UE_API FModelContextProtocolResource(const FString& Uri, const TArrayView<uint8>& BlobContent, const TOptional<FString>& Name = {}, const TOptional<FString>& Title = {}, const TOptional<FString>& MimeType = {});
};

/** List of Model Context Protocol resource descriptors that also tracks which IModelContextProtocolResourceProvider listed them */
struct FModelContextProtocolResourceDescriptorList
{
	/** Add a descriptor for a resource which ResourceProvider can provide */
	UE_API void Add(
		const FModelContextProtocolResourceDescriptor& ResourceDescriptor,
		const TSharedRef<const IModelContextProtocolResourceProvider>& ResourceProvider);

	/**
	 * Clears the list, including mapping of which provider added which descriptor.
	 * Note: To release only the cached Json descriptor list, use ReleaseJsonArray.   
	 */
	UE_API void Reset();

	/** Returns the number of descriptors in the list. */
	UE_API int32 Num() const;

	/** Returns the cached JSON list of resource descriptors */  
	UE_API TSharedRef<FJsonValueArray> GetJsonArray() const;

	/**
	 * Releases the cached JSON list of resource descriptors. Importantly, the mapping of which provider listed which URI is left intact for later
	 * URI to provider matching.
	 */
	UE_API void ReleaseJsonArray();

	/** Returns the resource provider that added Uri to this list (if a resource descriptor for Uri has been added, nullptr otherwise) */
	UE_API TSharedPtr<const IModelContextProtocolResourceProvider> FindResourceProvider(const FString& Uri) const;

protected:
	TMap<FString, TSharedRef<const IModelContextProtocolResourceProvider>> ResourceUriToProvider;
	FJsonDomBuilder::FArray JsonResourceDescriptorArray;
};

#undef UE_API
