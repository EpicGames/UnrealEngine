// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocolResources.h"
#include "ModelContextProtocolSession.h"

#include "Containers/Ticker.h"
#include "HttpResultCallback.h"
#include "HttpRouteHandle.h"
#include "Templates/SharedPointer.h"

class IHttpRouter;
class FJsonValue;
class FJsonObject;

namespace UE::ModelContextProtocol
{
	class FServerAccessor;
}

/**
 * MCP (Anthropic's Model Context Protocol) tool server. Serves MCP tools over HTTP per the https://modelcontextprotocol.io/ spec.
 * @see https://modelcontextprotocol.io
 * @see UModelContextProtocolSettings
 */
class FModelContextProtocolServer
{
	// Expose this class to a test accessor object.
	friend class UE::ModelContextProtocol::FServerAccessor;

public:

	FModelContextProtocolServer() = default;
	MODELCONTEXTPROTOCOL_API ~FModelContextProtocolServer();

	FModelContextProtocolServer(const FModelContextProtocolServer&) = delete;
	FModelContextProtocolServer& operator=(const FModelContextProtocolServer&) = delete;

	/**
	 * Starts the HTTP MCP server on the specified port and URL path.
	 */
	MODELCONTEXTPROTOCOL_API void StartServer(uint32 Port, const FString& UrlPath);

	/**
	 * Stops the HTTP MCP server.
	 */
	MODELCONTEXTPROTOCOL_API void StopServer();

	/** Returns true if the HTTP MCP server is currently running. */
	MODELCONTEXTPROTOCOL_API bool IsServerRunning() const;

	/** Returns the port the server is currently listening on, or 0 if not running. */
	MODELCONTEXTPROTOCOL_API uint32 GetServerPort() const;

	/** Schedules a notifications/tools/list_changed broadcast to all initialized sessions with active SSE streams on the next Tick. */
	MODELCONTEXTPROTOCOL_API void ScheduleToolsListChangedBroadcast();

private:

	void Tick(float DeltaTime);

	/** Set by ScheduleToolsListChangedBroadcast; drained on the next Tick. */
	bool bToolsListChangedBroadcastScheduled = false;

	TArray<TSharedPtr<FModelContextProtocolSession>> Sessions;

	TSharedPtr<FModelContextProtocolSession> FindSession(const FString& SessionId) const;

	TSharedPtr<IHttpRouter> HttpRouter;
	uint32 ActiveServerPort = 0;

	FHttpRouteHandle MainMcpRoute;
	FHttpRouteHandle SseMcpRoute;
	FHttpRouteHandle DeleteMcpRoute;

	FTSTicker::FDelegateHandle TickerHandle;

	/** Prevent use-after-free: async tool completion lambdas capture a TWeakPtr to this; Reset() in destructor invalidates all weak references. */
	TSharedPtr<bool> AliveGuard = MakeShared<bool>(true);

	FModelContextProtocolResourceDescriptorList LastResourceDescriptorList;

	bool ProcessPostRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool ProcessGetRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool ProcessDeleteRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

	bool ProcessJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const FString& Method, const TSharedPtr<FJsonObject>& Params, const FHttpResultCallback& OnComplete);
	bool ProcessPingJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const FHttpResultCallback& OnComplete);
	bool ProcessInitializeJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const FHttpResultCallback& OnComplete);
	bool ProcessInitializedNotificationJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete);
	bool ProcessNotificationCancelledJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete);
	bool ProcessListToolsJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete);
	bool ProcessToolCallJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete);
	bool ProcessListResourcesJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete);
	bool ProcessReadResourceJsonRpcCall(const FHttpServerRequest& Request, const TSharedPtr<FJsonValue>& RequestId, const TSharedPtr<FJsonObject>& Params, const TSharedRef<FModelContextProtocolSession>& Session, const FHttpResultCallback& OnComplete);
};
