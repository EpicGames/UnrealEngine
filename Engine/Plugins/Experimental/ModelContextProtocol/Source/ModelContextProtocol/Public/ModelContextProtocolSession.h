// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocol.h"
#include "ModelContextProtocolCapabilities.h"

#include "HttpResultCallback.h"
#include "Templates/SharedPointer.h"

#include "ModelContextProtocolSession.generated.h"

class FInternetAddr;
class FJsonValue;
struct IModelContextProtocolTool;

UENUM()
enum struct EModelContextProtocolSessionStatus
{
	/** Server and client are handshaking */
	Initializing,
	
	/** Server and client are both initialized */
	Initialized,
};

USTRUCT()
struct FModelContextProtocolClientInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString Title;

	UPROPERTY()
	FString Version;
};

USTRUCT()
struct FModelContextProtocolServerInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString Title;

	UPROPERTY()
	FString Version;
};

USTRUCT()
struct FModelContextProtocolPingResult
{
	GENERATED_BODY()
};

USTRUCT()
struct FModelContextProtocolInitializeParams
{
	GENERATED_BODY()

	UPROPERTY()
	FString ProtocolVersion = UE::ModelContextProtocol::ProtocolVersion;

	UPROPERTY()
	FModelContextProtocolClientCapabilities Capabilities;

	UPROPERTY()
	FModelContextProtocolClientInfo ClientInfo;
};

USTRUCT()
struct FModelContextProtocolInitializeResult
{
	GENERATED_BODY()

	UPROPERTY()
	FString ProtocolVersion = UE::ModelContextProtocol::ProtocolVersion;

	UPROPERTY()
	FModelContextProtocolServerCapabilities Capabilities;

	UPROPERTY()
	FModelContextProtocolServerInfo ServerInfo;

	UPROPERTY()
	TOptional<FString> Instructions;
};

struct FModelContextProtocolToolRequestId
{
	FModelContextProtocolToolRequestId() = default;
	explicit FModelContextProtocolToolRequestId(const TSharedPtr<FJsonValue>& InRequestId) : RequestId(InRequestId) {}

	TSharedPtr<FJsonValue> RequestId;

	bool IsValid() const { return RequestId.IsValid(); }

	bool operator==(const FModelContextProtocolToolRequestId& RHS) const;

	bool operator!=(const FModelContextProtocolToolRequestId& RHS) const
	{
		return !operator==(RHS);
	}

	friend uint32 GetTypeHash(const FModelContextProtocolToolRequestId& InRequestId);
};

struct FModelContextProtocolToolContext
{
	FModelContextProtocolToolContext() = default;

	TSharedPtr<IModelContextProtocolTool> Tool;
	TSharedPtr<FJsonValue> ProgressToken;

	FHttpResultCallback EventStreamWrite;

	double LastProgressSeconds = 0.0;
	int32 LastProgressValue = 0;
};

struct FModelContextProtocolSession : TSharedFromThis<FModelContextProtocolSession>
{
	FModelContextProtocolSession() = default;

	FString ID;
	EModelContextProtocolSessionStatus Status = EModelContextProtocolSessionStatus::Initializing;
	FString NegotiatedProtocolVersion = UE::ModelContextProtocol::ProtocolVersion;
	TSharedPtr<FInternetAddr> ClientAddress;
	FModelContextProtocolClientCapabilities ClientCapabilities;

	TMap<FModelContextProtocolToolRequestId, FModelContextProtocolToolContext> ActiveRequests;
};
