// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "JsonDomBuilder.h"
#include "JsonObjectConverter.h"
#include "JsonObjectWrapper.h"
#include "StructUtils/StructView.h"

#include "ModelContextProtocolToolResults.generated.h"

#define UE_API MODELCONTEXTPROTOCOL_API

struct FModelContextProtocolResourceDescriptor;

UENUM(meta = (Bitflags))
enum class EModelContextProtocolToolResultType
{
	None = 0,
	Text = 1 << 0,
	Image = 1 << 1,
	Audio = 1 << 2,
	ResourceLink = 1 << 3 UMETA(Hidden), 
	EmbeddedResource = 1 << 4 UMETA(Hidden),
	StructuredContent = 1 << 5
};
ENUM_CLASS_FLAGS(EModelContextProtocolToolResultType)

UENUM(meta = (Bitflags))
enum class EModelContextProtocolAudience
{
	None = 0,
	User = 1 << 0,
	Assistant = 1 << 1,
	All = User | Assistant
};
ENUM_CLASS_FLAGS(EModelContextProtocolAudience)

/**
 * IModelContextProtocolTool execution result.
 * 
 * Wraps a JSON object adhering to https://modelcontextprotocol.io/specification/2025-06-18/server/tools#tool-result spec.
 * 
 * Use MakeTextResult, MakeStructuredContentResult, MakeImageResult & MakeAudioResult for creating the appropriate tool result structures.
 */
USTRUCT(BlueprintType)
struct FModelContextProtocolToolResult : public FJsonObjectWrapper
{
	GENERATED_BODY()
	
	FModelContextProtocolToolResult() = default;
	FModelContextProtocolToolResult(const TSharedPtr<FJsonObject>& InJsonObject)
	{
		JsonObject = InJsonObject;
	}
};

namespace UE::ModelContextProtocol
{
	// @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools#text-content
	template <typename TextT>
	TSharedPtr<FJsonObject> MakeTextContentObject(TextT Text)
	{
		FJsonDomBuilder::FObject TextContentObject;
		TextContentObject.Set(TEXT("type"), TEXT("text"));
		TextContentObject.Set(TEXT("text"), Text);
		return TextContentObject.AsJsonObject();
	}

	template <typename TextT>
	FModelContextProtocolToolResult MakeTextResult(TextT Text)
	{
		FJsonDomBuilder::FArray ContentArray;
		ContentArray.Add(MakeTextContentObject(Text));
		FJsonDomBuilder::FObject ResultObject;
		ResultObject.Set(TEXT("content"), ContentArray);
		return ResultObject.AsJsonObject().ToSharedPtr();
	}

	// @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools#image-content
	UE_API FModelContextProtocolToolResult MakeImageResult(const FString& MimeType, TArrayView<uint8> Data, EModelContextProtocolAudience Audience = EModelContextProtocolAudience::All);
	UE_API FModelContextProtocolToolResult MakeImageResult(const FString& MimeType, FString&& Base64EncodedData, EModelContextProtocolAudience Audience = EModelContextProtocolAudience::All);

	// @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools#audio-content
	UE_API FModelContextProtocolToolResult MakeAudioResult(const FString& MimeType, TArrayView<uint8> Data);
	UE_API FModelContextProtocolToolResult MakeAudioResult(const FString& MimeType, FString&& Base64EncodedData);

	// @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools#resource-links
	UE_API FModelContextProtocolToolResult MakeResourceLinkResult(const FModelContextProtocolResourceDescriptor& ResourceDescriptor, EModelContextProtocolAudience Audience = EModelContextProtocolAudience::All);
	UE_API FModelContextProtocolToolResult MakeResourceLinksResult(TConstArrayView<FModelContextProtocolResourceDescriptor> ResourceDescriptors, EModelContextProtocolAudience Audience = EModelContextProtocolAudience::All);

	struct FJsonExportParams
	{
		int64 CheckFlags = CPF_None;
		int64 SkipFlags = CPF_None;
		const FJsonObjectConverter::CustomExportCallback* ExportCb = nullptr;
		EJsonObjectConversionFlags ConversionFlags = EJsonObjectConversionFlags::None;
	};

	// @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools#structured-content
	UE_API FModelContextProtocolToolResult MakeStructuredContentResult(const UStruct* StructDefinition, const void* Struct, const FJsonExportParams& Params = {});
	template<typename T>
	FModelContextProtocolToolResult MakeStructuredContentResult(const T& Struct, const FJsonExportParams& Params = {})
	{
		return MakeStructuredContentResult(T::StaticStruct(), &Struct, Params);
	}
	FORCEINLINE FModelContextProtocolToolResult MakeStructuredContentResult(const UObject* Object, const FJsonExportParams& Params = {})
	{
		return MakeStructuredContentResult(Object->GetClass(), static_cast<const void*>(Object), Params);
	}
	UE_API FModelContextProtocolToolResult MakeStructuredContentResult(FProperty* Property, const void* Value, const FJsonExportParams& Params = {});
	UE_API FModelContextProtocolToolResult MakeStructuredContentResult(TSharedPtr<FJsonValue> StructuredContent);

	// @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools#error-handling
	template <typename TextT>
	FModelContextProtocolToolResult MakeErrorResult(TextT Text)
	{
		FModelContextProtocolToolResult Result = MakeTextResult(Text);
		Result.JsonObject->SetBoolField(TEXT("isError"), true);
		return Result;
	}
}

#undef UE_API
