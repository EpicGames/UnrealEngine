// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Paths.h"

#define UE_API MODELCONTEXTPROTOCOL_API

namespace UE::ModelContextProtocol
{
	/**
	 * Joins BasePath & Filename, ensuring the resulting path is indeed under BasePath.
	 * Useful to ensure models can't create files outside the intended BasePath using relative directory traversal e.g YourBasePath/../SomeOtherBase/MyFile.ext
	 */
	bool SafeConvertRelativePathToFull(const FString& BasePath, const FString& FileName, FString& OutFilePath)
	{
		OutFilePath = FPaths::ConvertRelativePathToFull(BasePath, FileName);
		if (!FPaths::IsUnderDirectory(OutFilePath, BasePath))
		{
			OutFilePath.Reset();
			return false;
		}
		return true;
	}
}

#undef UE_API
