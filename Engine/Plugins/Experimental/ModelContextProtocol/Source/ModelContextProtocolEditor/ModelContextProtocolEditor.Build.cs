// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelContextProtocolEditor : ModuleRules
{
	public ModelContextProtocolEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"AssetDefinition",
				"EngineAssetDefinitions",
				"ModelContextProtocol",
				"ModelContextProtocolEngine",
				"Blutility"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"ToolMenus",
				"CoreUObject",
				"Engine",
				"Json",
				"JsonUtilities",
				"Slate",
				"SlateCore",
				"BlueprintGraph",
				"DeveloperSettings",
				"JsonUtilitiesEditor",
				"KismetCompiler",
				"ToolsetRegistry"
			});
	}
}
