// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolAssetDefinitions.h"

#include "ModelContextProtocolEditorToolLibrary.h"
#include "ModelContextProtocolToolLibrary.h"

#define LOCTEXT_NAMESPACE "ModelContextProtocolAssetDefinitions"

FText UAssetDefinition_ModelContextProtocolToolLibraryBlueprint::GetAssetDisplayName() const
{
	return LOCTEXT("ModelContextProtocolToolLibraryBlueprintName", "MCP Tool Library Blueprint");
}

FLinearColor UAssetDefinition_ModelContextProtocolToolLibraryBlueprint::GetAssetColor() const
{
	return FColor(255, 195, 0);
}

TSoftClassPtr<> UAssetDefinition_ModelContextProtocolToolLibraryBlueprint::GetAssetClass() const
{
	return UModelContextProtocolToolLibraryBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ModelContextProtocolToolLibraryBlueprint::GetAssetCategories() const
{
	static const auto Categories = 
	{
		FAssetCategoryPath(EAssetCategoryPaths::AI, LOCTEXT("AssetDefinition_ModelContextProtocolToolLibrarySubMenu", "Generative AI"), ECategoryMenuType::Section)
	};
	return Categories;
}

FText UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint::GetAssetDisplayName() const
{
	return LOCTEXT("ModelContextProtocolEditorToolLibraryBlueprintName", "MCP EditorTool Library Blueprint");
}

FLinearColor UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint::GetAssetColor() const
{
	return FColor(0, 169, 255);
}

TSoftClassPtr<> UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint::GetAssetClass() const
{
	return UModelContextProtocolEditorToolLibraryBlueprint::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint::GetAssetCategories() const
{
	static const auto Categories = 
	{
		FAssetCategoryPath(EAssetCategoryPaths::AI, LOCTEXT("AssetDefinition_ModelContextProtocolEditorToolLibrarySubMenu", "Generative AI"), ECategoryMenuType::Section)
	};
	return Categories;
}

#undef LOCTEXT_NAMESPACE
