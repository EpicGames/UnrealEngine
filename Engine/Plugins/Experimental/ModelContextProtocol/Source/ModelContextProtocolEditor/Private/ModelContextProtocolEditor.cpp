// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolEditor.h"
#include "IModelContextProtocolModule.h"
#include "ModelContextProtocolEditorToolLibrary.h"
#include "ModelContextProtocolSettings.h"
#include "ModelContextProtocolToolLibrary.h"

#include "Editor.h"
#include "KismetCompilerModule.h"
#include "ToolsetRegistry/Private/ToolsetRegistry/EngineDelegatesCompatibility.h"
#include "ToolsetRegistry/ToolsetRegistry.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"

#define LOCTEXT_NAMESPACE "FModelContextProtocolEditorModule"

void FModelContextProtocolEditorModule::StartupModule()
{
	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.OverrideBPTypeForClass(UModelContextProtocolToolLibrary::StaticClass(), UModelContextProtocolToolLibraryBlueprint::StaticClass());
	KismetCompilerModule.OverrideBPTypeForClass(UModelContextProtocolEditorToolLibrary::StaticClass(), UModelContextProtocolEditorToolLibraryBlueprint::StaticClass());

	// Re-register adapters when MCP tools are refreshed
	if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
	{
		OnRefreshToolsHandle = Module->OnRefreshTools().AddLambda([this]()
		{
			ToolsetRegistryAdapterManager.RegisterTools();
		});
	}

	// If GEditor is already available, set up immediately. Otherwise, defer until post-engine-init.
	// Uses FOnPostEngineInit compatibility shim to support branches that lack FCoreDelegates::GetOnPostEngineInit().
	if (GEditor)
	{
		SetupEditorIntegration();
	}
	else
	{
		FSimpleMulticastDelegate& OnPostEngineInit = FOnPostEngineInit<FCoreDelegates>::Get();
		PostEngineInitHandle = UE::ToolsetRegistry::FDelegateHandleRaii::Create(
			OnPostEngineInit, OnPostEngineInit.AddLambda([this]()
			{
				SetupEditorIntegration();
			}));
	}
}

void FModelContextProtocolEditorModule::SetupEditorIntegration()
{
	// Subscribe to ToolsetRegistry changes so adapters are re-registered when toolsets are added
	if (UToolsetRegistrySubsystem* ToolsetRegistrySubsystem = GEditor ? GEditor->GetEditorSubsystem<UToolsetRegistrySubsystem>() : nullptr)
	{
		OnToolsetRegisteredHandle = ToolsetRegistrySubsystem->ToolsetRegistry.OnToolsetRegistered().AddLambda([this]()
		{
			ToolsetRegistryAdapterManager.RegisterTools();
		});
	}

	// Initial registration of any toolsets already loaded
	ToolsetRegistryAdapterManager.RegisterTools();

	// Auto-start the MCP server if configured
	if (UE::ModelContextProtocol::ShouldAutoStartServer())
	{
		if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
		{
			Module->StartServer(UE::ModelContextProtocol::GetServerPortNumber(), UE::ModelContextProtocol::GetServerUrlPath());
		}
	}
}

void FModelContextProtocolEditorModule::ShutdownModule()
{
	PostEngineInitHandle.Reset();

	if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
	{
		Module->OnRefreshTools().Remove(OnRefreshToolsHandle);
	}

	if (UToolsetRegistrySubsystem* ToolsetRegistrySubsystem = GEditor ? GEditor->GetEditorSubsystem<UToolsetRegistrySubsystem>() : nullptr)
	{
		ToolsetRegistrySubsystem->ToolsetRegistry.OnToolsetRegistered().Remove(OnToolsetRegisteredHandle);
	}

	ToolsetRegistryAdapterManager.DeregisterTools();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FModelContextProtocolEditorModule, ModelContextProtocolEditor)
