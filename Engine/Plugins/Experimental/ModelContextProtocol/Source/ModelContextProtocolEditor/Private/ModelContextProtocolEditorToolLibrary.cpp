// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolEditorToolLibrary.h"
#include "ModelContextProtocolToolLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelContextProtocolEditorToolLibrary)

#define LOCTEXT_NAMESPACE "ModelContextProtocolEditorToolLibrary"

UModelContextProtocolEditorToolLibraryBlueprint::UModelContextProtocolEditorToolLibraryBlueprint()
{
	BlueprintType = BPTYPE_FunctionLibrary;
	ParentClass = UModelContextProtocolEditorToolLibrary::StaticClass();
#if WITH_EDITORONLY_DATA
	ShouldCookPropertyGuidsValue = EShouldCookBlueprintPropertyGuids::Yes;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
bool UModelContextProtocolEditorToolLibraryBlueprint::AlwaysCompileOnLoad() const
{
	return true;
}
#endif

#undef LOCTEXT_NAMESPACE
