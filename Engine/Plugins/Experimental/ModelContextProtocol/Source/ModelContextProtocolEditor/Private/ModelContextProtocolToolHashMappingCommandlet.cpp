// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolHashMappingCommandlet.h"

#include "IModelContextProtocolModule.h"
#include "IModelContextProtocolTool.h"
#include "ModelContextProtocolAnalytics.h"
#include "ModelContextProtocolToolSearch.h"
#include "ModelContextProtocolSettings.h"

#include "Commandlets/Commandlet.h"
#include "Containers/SortedMap.h"
#include "Containers/Set.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Logging/LogMacros.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Serialization/JsonWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelContextProtocolToolHashMappingCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogModelContextProtocolToolHashMapping, Log, All);

namespace
{
	inline constexpr TCHAR MappingDescription[] = TEXT("blake3(utf-8) hashes of MCP tool and toolset identifiers. See UE::ModelContextProtocol::Analytics::HashToolIdentifier.");

	FString ResolveOutputPath(const FString& Params)
	{
		FString OutputPath;
		FParse::Value(*Params, TEXT("Output="), OutputPath);
		if (OutputPath.IsEmpty())
		{
			OutputPath = FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("ModelContextProtocol"), TEXT("ToolHashMapping.json"));
		}
		return FPaths::ConvertRelativePathToFull(OutputPath);
	}

	void DisableToolSearch()
	{
		GetMutableDefault<UModelContextProtocolSettings>()->bEnableToolSearch = false;
	}

	void CollectToolNames(const IModelContextProtocolModule& Module, TSet<FString>& OutNames)
	{
		for (const TSharedRef<IModelContextProtocolTool>& Tool : Module.GetTools())
		{
			OutNames.Add(Tool->GetName());
		}
		// Tool-search-mode meta-tools never appear in eager-mode GetTools() but must be in the mapping so analytics records from tool-search sessions can be decoded.
		OutNames.Add(UE::ModelContextProtocol::ListToolsetsName);
		OutNames.Add(UE::ModelContextProtocol::DescribeToolsetName);
		OutNames.Add(UE::ModelContextProtocol::CallToolName);
	}

	FString SerializeMapping(const TSortedMap<FString, FString>& Tools, const TSortedMap<FString, FString>& Toolsets)
	{
		FString Output;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&Output);

		Writer->WriteObjectStart();
		Writer->WriteValue(TEXT("description"), MappingDescription);
		Writer->WriteValue(TEXT("toolCount"), Tools.Num());
		Writer->WriteValue(TEXT("toolsetCount"), Toolsets.Num());

		Writer->WriteObjectStart(TEXT("tools"));
		for (const TPair<FString, FString>& Entry : Tools)
		{
			Writer->WriteObjectStart(Entry.Key);
			Writer->WriteValue(TEXT("fullName"), Entry.Value);
			Writer->WriteObjectEnd();
		}
		Writer->WriteObjectEnd();

		Writer->WriteObjectStart(TEXT("toolsets"));
		for (const TPair<FString, FString>& Entry : Toolsets)
		{
			Writer->WriteObjectStart(Entry.Key);
			Writer->WriteValue(TEXT("toolsetName"), Entry.Value);
			Writer->WriteObjectEnd();
		}
		Writer->WriteObjectEnd();

		Writer->WriteObjectEnd();
		Writer->Close();

		return Output;
	}
}

void UModelContextProtocolToolHashMappingCommandlet::BuildHashMaps(const TSet<FString>& Names, TSortedMap<FString, FString>& OutTools, TSortedMap<FString, FString>& OutToolsets)
{
	bool bHasTopLevelTool = false;

	for (const FString& Name : Names)
	{
		OutTools.Add(UE::ModelContextProtocol::Analytics::HashToolIdentifier(Name), Name);

		const FString Prefix = UE::ModelContextProtocol::Analytics::ParseToolsetName(Name);
		if (Prefix.IsEmpty())
		{
			bHasTopLevelTool = true;
			continue;
		}
		// Multiple tools can share a toolset prefix; FindOrAdd deduplicates.
		OutToolsets.FindOrAdd(UE::ModelContextProtocol::Analytics::HashToolIdentifier(Prefix)) = Prefix;
	}

	// Top-level tools (no dot in the name) hash against an empty toolset prefix; include that entry so analytics rows for such tools decode.
	if (bHasTopLevelTool)
	{
		OutToolsets.FindOrAdd(UE::ModelContextProtocol::Analytics::HashToolIdentifier(FString())) = FString();
	}
}

int32 UModelContextProtocolToolHashMappingCommandlet::Main(const FString& Params)
{
	// Match the convention of other ticking commandlets (e.g. UFileServerCommandlet): mark the engine as running so subsystems that gate behavior on `GIsRunning` advance correctly during the manual tick loop below.
	GIsRunning = true;

	IModelContextProtocolModule* Module = IModelContextProtocolModule::Get();
	if (!Module)
	{
		UE_LOGF(LogModelContextProtocolToolHashMapping, Error, "IModelContextProtocolModule is not available; cannot enumerate tools.");
		return 1;
	}

	DisableToolSearch();

	// Commandlets do not run the normal editor tick loop, so any async work kicked off by -ExecCmds (e.g. GameFeature.Activate) would be frozen mid-load when we enumerate. Manually drive engine frames for a configurable window so GFP activations and their downstream OnToolsetRegistered -> MCP adapter re-registration callbacks can complete.
	float TickEngineSeconds = 30.0f;
	FParse::Value(*Params, TEXT("TickEngineSeconds="), TickEngineSeconds);
	if (TickEngineSeconds > 0.0f)
	{
		UE_LOGF(LogModelContextProtocolToolHashMapping, Display, "Ticking the engine for %.2fs to let -ExecCmds, game feature activations, and toolset registrations complete.", TickEngineSeconds);

		const double EndTime = FPlatformTime::Seconds() + TickEngineSeconds;
		while (FPlatformTime::Seconds() < EndTime && !IsEngineExitRequested())
		{
			constexpr float DeltaTime = 0.05f;
			CommandletHelpers::TickEngine(nullptr, DeltaTime);
			FPlatformProcess::Sleep(DeltaTime);
		}
	}

	// Finalize tool-search-disabled state: even if something activated a toolset in tool-search mode before we flipped the setting, this re-broadcast forces every OnRefreshTools subscriber (notably the ToolsetRegistry adapter manager) to re-register with tool search disabled.
	Module->RefreshTools();

	TSet<FString> Names;
	CollectToolNames(*Module, Names);
	if (Names.IsEmpty())
	{
		UE_LOGF(LogModelContextProtocolToolHashMapping, Error, "Module->GetTools() returned no tools after RefreshTools().");
		return 1;
	}

	TSortedMap<FString, FString> Tools;
	TSortedMap<FString, FString> Toolsets;
	BuildHashMaps(Names, Tools, Toolsets);

	const FString OutputPath = ResolveOutputPath(Params);
	const FString OutputDir = FPaths::GetPath(OutputPath);
	if (!IFileManager::Get().MakeDirectory(*OutputDir, /*Tree=*/true))
	{
		UE_LOGF(LogModelContextProtocolToolHashMapping, Error, "Failed to create output directory: %ls", *OutputDir);
		return 1;
	}

	const FString Json = SerializeMapping(Tools, Toolsets);
	if (!FFileHelper::SaveStringToFile(Json, *OutputPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		UE_LOGF(LogModelContextProtocolToolHashMapping, Error, "Failed to write mapping to: %ls", *OutputPath);
		return 1;
	}

	UE_LOGF(LogModelContextProtocolToolHashMapping, Display, "Wrote %d tool hashes and %d toolset hashes to %ls", Tools.Num(), Toolsets.Num(), *OutputPath);

	if (FParse::Param(*Params, TEXT("Print")))
	{
		UE_LOGF(LogModelContextProtocolToolHashMapping, Display, "%ls", *Json);
	}

	return 0;
}
