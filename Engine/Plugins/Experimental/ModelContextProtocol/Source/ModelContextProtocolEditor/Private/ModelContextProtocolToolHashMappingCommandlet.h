// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "Containers/Set.h"
#include "Containers/SortedMap.h"
#include "Containers/UnrealString.h"

#include "ModelContextProtocolToolHashMappingCommandlet.generated.h"

/**
 * Writes a JSON mapping of blake3 tool and toolset identifier hashes (as emitted by UE::ModelContextProtocol::Analytics::HashToolIdentifier) to their human-readable names. The mapping lets downstream consumers decode ToolNameHash / ToolsetNameHash fields in analytics events.
 *
 * Usage:
 *   UnrealEditor-Cmd.exe <uproject> <TargetName> -run=ModelContextProtocolToolHashMapping [-Output=<path>] [-TickEngineSeconds=<float>] [-Print]
 *
 * Default output path is <ProjectSavedDir>/ModelContextProtocol/ToolHashMapping.json.
 *
 * Only tools registered with IModelContextProtocolModule at enumeration time are captured.
 *
 * Toolsets packaged as explicitly-loaded Game Feature Plugins are not active by default and must be activated before enumeration via -ExecCmds (use the `LoadGameFeaturePlugin` console command).
 *
 * Because commandlets do not run the normal editor tick loop, the commandlet manually drives engine frames for -TickEngineSeconds (default 20.0) so async GFP activations and their downstream toolset registrations can complete before the mapping is built. Pass -TickEngineSeconds=0 to skip the loop entirely. The default is sized for a typical GFP activation; bump it higher for larger aggregates or cold caches.
 *
 * Example that activates a toolset aggregate GFP before enumerating:
 *   UnrealEditor-Cmd.exe <uproject> <EditorTarget> -run=ModelContextProtocolToolHashMapping -ExecCmds="LoadGameFeaturePlugin <YourToolsetsPluginName>"
 */
UCLASS()
class UModelContextProtocolToolHashMappingCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface

	/**
	 * Builds the hash -> name maps from a deduplicated set of tool names. Exposed as a static method so tests can exercise the split-at-last-dot, top-level-sentinel, and sorted-output behavior without driving a full commandlet run.
	 */
	static MODELCONTEXTPROTOCOLEDITOR_API void BuildHashMaps(const TSet<FString>& Names, TSortedMap<FString, FString>& OutTools, TSortedMap<FString, FString>& OutToolsets);
};
