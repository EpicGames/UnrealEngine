// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolLibraryFactory.h"
#include "ModelContextProtocolEditorToolLibrary.h"
#include "ModelContextProtocolToolLibrary.h"

#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/MessageDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelContextProtocolToolLibraryFactory)

#define LOCTEXT_NAMESPACE "ModelContextProtocolToolLibraryFactory"

UModelContextProtocolToolLibraryFactory::UModelContextProtocolToolLibraryFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	// static FBoolConfigValueHelper CanCreateNewHelper(TEXT("CustomModelContextProtocolToolLibrary"), TEXT("bCanCreateNew"));
	// bCreateNew = CanCreateNewHelper;
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UModelContextProtocolToolLibraryBlueprint::StaticClass();
	ParentClass = UModelContextProtocolToolLibrary::StaticClass();
	BlueprintType = BPTYPE_FunctionLibrary;
}

FText UModelContextProtocolToolLibraryFactory::GetDisplayName() const
{
	return LOCTEXT("ModelContextProtocolToolLibraryFactoryDescription", "MCP Tool Library");
}

FName UModelContextProtocolToolLibraryFactory::GetNewAssetThumbnailOverride() const
{
	return TEXT("ClassThumbnail.BlueprintFunctionLibrary");
}

FText UModelContextProtocolToolLibraryFactory::GetToolTip() const
{
	return LOCTEXT("BlueprintFunctionLibraryTooltip", "MCP (Anthropic's Model Context Protocol) Tool Libraries are containers of functions which will be automatically exposed as MCP tools for LLM's to use.");
}

UObject* UModelContextProtocolToolLibraryFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a blueprint, then create and init one
	check(Class->IsChildOf(UModelContextProtocolToolLibraryBlueprint::StaticClass()));

	if (ParentClass != UModelContextProtocolToolLibrary::StaticClass())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString(ParentClass->GetName()) : LOCTEXT("Null", "(null)"));
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("CannotCreateBlueprintFromClass", "Cannot create a blueprint based on the class '{ClassName}'."), Args));
		return nullptr;
	}
	else
	{
		return FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UModelContextProtocolToolLibraryBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext);
	}
}

bool UModelContextProtocolToolLibraryFactory::ConfigureProperties()
{
	return true;
}

FString UModelContextProtocolToolLibraryFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("ModelContextProtocolTools"));
}

UModelContextProtocolEditorToolLibraryFactory::UModelContextProtocolEditorToolLibraryFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	SupportedClass = UModelContextProtocolEditorToolLibraryBlueprint::StaticClass();
	ParentClass = UModelContextProtocolEditorToolLibrary::StaticClass();
}

FText UModelContextProtocolEditorToolLibraryFactory::GetDisplayName() const
{
	return LOCTEXT("ModelContextProtocolEditorToolLibraryFactoryDescription", "MCP Editor Tool Library");
}

UObject* UModelContextProtocolEditorToolLibraryFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext)
{
	// Make sure we are trying to factory a blueprint, then create and init one
	check(Class->IsChildOf(UModelContextProtocolEditorToolLibraryBlueprint::StaticClass()));

	if (ParentClass != UModelContextProtocolEditorToolLibrary::StaticClass())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ClassName"), (ParentClass != nullptr) ? FText::FromString(ParentClass->GetName()) : LOCTEXT("Null", "(null)"));
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("CannotCreateBlueprintFromClass", "Cannot create a blueprint based on the class '{ClassName}'."), Args));
		return nullptr;
	}
	else
	{
		return FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, Name, BlueprintType, UModelContextProtocolEditorToolLibraryBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), CallingContext);
	}
}

FString UModelContextProtocolEditorToolLibraryFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("ModelContextProtocolEditorTools"));
}

#undef LOCTEXT_NAMESPACE
