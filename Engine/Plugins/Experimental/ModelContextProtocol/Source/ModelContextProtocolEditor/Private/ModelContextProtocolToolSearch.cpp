// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolSearch.h"

#include "Dom/JsonObject.h"
#include "ModelContextProtocol.h"
#include "ModelContextProtocolToolResults.h"

namespace UE::ModelContextProtocol::Private
{
	TSharedPtr<FJsonObject> MakeToolsetNameSchema(const FString& Description)
	{
		TSharedPtr<FJsonObject> Schema = MakeShared<FJsonObject>();
		Schema->SetStringField(TEXT("type"), TEXT("object"));

		TSharedPtr<FJsonObject> Properties = MakeShared<FJsonObject>();
		TSharedPtr<FJsonObject> ToolsetNameProp = MakeShared<FJsonObject>();
		ToolsetNameProp->SetStringField(TEXT("type"), TEXT("string"));
		ToolsetNameProp->SetStringField(TEXT("description"), Description);
		Properties->SetObjectField(TEXT("toolset_name"), ToolsetNameProp);
		Schema->SetObjectField(TEXT("properties"), Properties);

		TArray<TSharedPtr<FJsonValue>> RequiredArray;
		RequiredArray.Add(MakeShared<FJsonValueString>(TEXT("toolset_name")));
		Schema->SetArrayField(TEXT("required"), RequiredArray);

		return Schema;
	}
}

// -- FListToolsetsTool --

FListToolsetsTool::FListToolsetsTool(FListToolsetsDelegate InDelegate)
	: ListDelegate(MoveTemp(InDelegate))
{
}

TSharedPtr<FJsonObject> FListToolsetsTool::GetInputJsonSchema() const
{
	TSharedPtr<FJsonObject> Schema = MakeShared<FJsonObject>();
	Schema->SetStringField(TEXT("type"), TEXT("object"));
	Schema->SetObjectField(TEXT("properties"), MakeShared<FJsonObject>());
	return Schema;
}

FModelContextProtocolToolResult FListToolsetsTool::Run(const TSharedPtr<FJsonObject>& Params)
{
	FString Result = ListDelegate();
	return UE::ModelContextProtocol::MakeTextResult(Result);
}

// -- FDescribeToolsetTool --

FDescribeToolsetTool::FDescribeToolsetTool(FDescribeToolsetDelegate InDelegate)
	: DescribeDelegate(MoveTemp(InDelegate))
{
}

TSharedPtr<FJsonObject> FDescribeToolsetTool::GetInputJsonSchema() const
{
	return UE::ModelContextProtocol::Private::MakeToolsetNameSchema(TEXT("Name of the toolset to describe. Use list_toolsets to see available names."));
}

FModelContextProtocolToolResult FDescribeToolsetTool::Run(const TSharedPtr<FJsonObject>& Params)
{
	if (!Params.IsValid())
	{
		return UE::ModelContextProtocol::MakeErrorResult(TEXT("Missing parameters."));
	}

	FString ToolsetName;
	if (!Params->TryGetStringField(TEXT("toolset_name"), ToolsetName) || ToolsetName.IsEmpty())
	{
		return UE::ModelContextProtocol::MakeErrorResult(TEXT("Missing required parameter: toolset_name"));
	}

	TValueOrError<FString, FString> Result = DescribeDelegate(ToolsetName);
	if (Result.HasError())
	{
		return UE::ModelContextProtocol::MakeErrorResult(Result.GetError());
	}

	return UE::ModelContextProtocol::MakeTextResult(Result.GetValue());
}

// -- FCallTool --

FCallTool::FCallTool(FCallToolDelegate InDelegate)
	: CallDelegate(MoveTemp(InDelegate))
{
}

TSharedPtr<FJsonObject> FCallTool::GetInputJsonSchema() const
{
	TSharedPtr<FJsonObject> Schema = MakeShared<FJsonObject>();
	Schema->SetStringField(TEXT("type"), TEXT("object"));

	TSharedPtr<FJsonObject> Properties = MakeShared<FJsonObject>();

	TSharedPtr<FJsonObject> ToolsetNameProp = MakeShared<FJsonObject>();
	ToolsetNameProp->SetStringField(TEXT("type"), TEXT("string"));
	ToolsetNameProp->SetStringField(TEXT("description"), TEXT("Optional. Name of the toolset containing the tool. Omit to call a top-level MCP tool. Use list_toolsets to discover toolset names."));
	Properties->SetObjectField(TEXT("toolset_name"), ToolsetNameProp);

	TSharedPtr<FJsonObject> ToolNameProp = MakeShared<FJsonObject>();
	ToolNameProp->SetStringField(TEXT("type"), TEXT("string"));
	ToolNameProp->SetStringField(TEXT("description"), TEXT("Name of the tool to call, without toolset prefix. Use describe_toolset to discover tool names and input schemas."));
	Properties->SetObjectField(TEXT("tool_name"), ToolNameProp);

	TSharedPtr<FJsonObject> ArgumentsProp = MakeShared<FJsonObject>();
	ArgumentsProp->SetStringField(TEXT("type"), TEXT("object"));
	ArgumentsProp->SetStringField(TEXT("description"), TEXT("Arguments to pass to the tool. Must match the tool's input schema. Defaults to an empty object."));
	Properties->SetObjectField(TEXT("arguments"), ArgumentsProp);

	Schema->SetObjectField(TEXT("properties"), Properties);

	TArray<TSharedPtr<FJsonValue>> RequiredArray;
	RequiredArray.Add(MakeShared<FJsonValueString>(TEXT("tool_name")));
	Schema->SetArrayField(TEXT("required"), RequiredArray);

	return Schema;
}

void FCallTool::RunAsync(const FModelContextProtocolToolRequestId& RequestId, const TSharedPtr<FJsonObject>& Params, const FResultCallback& OnComplete)
{
	if (!Params.IsValid())
	{
		OnComplete(UE::ModelContextProtocol::MakeErrorResult(TEXT("Missing parameters.")));
		return;
	}

	FString ToolName;
	if (!Params->TryGetStringField(TEXT("tool_name"), ToolName) || ToolName.IsEmpty())
	{
		OnComplete(UE::ModelContextProtocol::MakeErrorResult(TEXT("Missing required parameter: tool_name")));
		return;
	}

	FString ToolsetName;
	Params->TryGetStringField(TEXT("toolset_name"), ToolsetName);

	const TSharedPtr<FJsonObject>* ArgumentsObject = nullptr;
	Params->TryGetObjectField(TEXT("arguments"), ArgumentsObject);
	TSharedPtr<FJsonObject> Arguments = (ArgumentsObject && ArgumentsObject->IsValid()) ? *ArgumentsObject : MakeShared<FJsonObject>();

	CallDelegate(ToolsetName, ToolName, Arguments, RequestId, OnComplete);
}
