// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IModelContextProtocolTool.h"
#include "Templates/ValueOrError.h"

namespace UE::ModelContextProtocol
{
	inline constexpr TCHAR ListToolsetsName[] = TEXT("list_toolsets");
	inline constexpr TCHAR DescribeToolsetName[] = TEXT("describe_toolset");
	inline constexpr TCHAR CallToolName[] = TEXT("call_tool");
}

/** Callback: returns a text listing of all available toolsets (name + description). */
using FListToolsetsDelegate = TFunction<FString()>;

/** Callback: returns the full JSON schema for a named toolset, or an error string. */
using FDescribeToolsetDelegate = TFunction<TValueOrError<FString, FString>(const FString& ToolsetName)>;

/** Callback: dispatches a tool call. ToolsetName may be empty (top-level tool). Arguments may be null (interpreted as empty object). Result is delivered asynchronously via OnComplete. */
using FCallToolDelegate = TFunction<void(const FString& ToolsetName, const FString& ToolName, const TSharedPtr<FJsonObject>& Arguments, const FModelContextProtocolToolRequestId& RequestId, const IModelContextProtocolTool::FResultCallback& OnComplete)>;

/**
 * Lists all available toolsets with their names and descriptions.
 * Use this to discover what toolsets are available before describing or calling their tools.
 */
struct FListToolsetsTool : IModelContextProtocolTool
{
	explicit FListToolsetsTool(FListToolsetsDelegate InDelegate);

	virtual FString GetName() const override { return UE::ModelContextProtocol::ListToolsetsName; }
	virtual FString GetDescription() const override { return TEXT("List all available toolsets with names and descriptions."); }
	virtual TSharedPtr<FJsonObject> GetInputJsonSchema() const override;
	virtual FModelContextProtocolToolResult Run(const TSharedPtr<FJsonObject>& Params) override;

private:
	FListToolsetsDelegate ListDelegate;
};

/**
 * Returns detailed information about a specific toolset including all tool names,
 * descriptions, and input schemas. Use this to inspect a toolset before calling its tools.
 */
struct FDescribeToolsetTool : IModelContextProtocolTool
{
	explicit FDescribeToolsetTool(FDescribeToolsetDelegate InDelegate);

	virtual FString GetName() const override { return UE::ModelContextProtocol::DescribeToolsetName; }
	virtual FString GetDescription() const override
	{
		return TEXT("Get detailed information about a toolset including all tool names, descriptions, and input schemas.");
	}
	virtual TSharedPtr<FJsonObject> GetInputJsonSchema() const override;
	virtual FModelContextProtocolToolResult Run(const TSharedPtr<FJsonObject>& Params) override;

private:
	FDescribeToolsetDelegate DescribeDelegate;
};

/**
 * Calls a single tool by name without registering it as a native MCP tool.
 * Provide toolset_name to call a tool inside a toolset; omit it to call a top-level MCP tool.
 * Use list_toolsets and describe_toolset to discover available tools and their input schemas.
 */
struct FCallTool : IModelContextProtocolTool
{
	explicit FCallTool(FCallToolDelegate InDelegate);

	virtual FString GetName() const override { return UE::ModelContextProtocol::CallToolName; }
	virtual FString GetDescription() const override
	{
		return TEXT("Call a tool by name. Provide toolset_name to call a toolset tool, or omit it to call a top-level MCP tool. Use list_toolsets and describe_toolset to discover available tools and their input schemas.");
	}
	virtual TSharedPtr<FJsonObject> GetInputJsonSchema() const override;
	virtual void RunAsync(const FModelContextProtocolToolRequestId& RequestId, const TSharedPtr<FJsonObject>& Params, const FResultCallback& OnComplete) override;

private:
	FCallToolDelegate CallDelegate;
};
