// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolsetRegistryAdapter.h"

#include "IModelContextProtocolModule.h"
#include "ModelContextProtocol.h"
#include "ModelContextProtocolToolSearch.h"
#include "ModelContextProtocolServer.h"
#include "ModelContextProtocolSettings.h"
#include "ModelContextProtocolToolResults.h"

#include "Editor.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "ToolsetRegistry/Toolset.h"
#include "ToolsetRegistry/ToolsetRegistry.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"

namespace UE::ModelContextProtocol::Private
{
	UE::ToolsetRegistry::FToolsetRegistry* GetToolsetRegistry()
	{
		UToolsetRegistrySubsystem* Subsystem = GEditor ?
			GEditor->GetEditorSubsystem<UToolsetRegistrySubsystem>() : nullptr;
		return Subsystem ? &Subsystem->ToolsetRegistry : nullptr;
	}
}

FToolsetRegistryToolAdapter::FToolsetRegistryToolAdapter(const FString& InToolName, const FString& InDescription, TSharedPtr<FJsonObject> InInputSchema)
	: ToolName(InToolName)
	, Description(InDescription)
	, InputSchema(MoveTemp(InInputSchema))
{
}

void FToolsetRegistryToolAdapter::RunAsync(const FModelContextProtocolToolRequestId& RequestId, const TSharedPtr<FJsonObject>& Params, const FResultCallback& OnComplete)
{
	UE::ToolsetRegistry::FToolsetRegistry* ToolsetRegistry = UE::ModelContextProtocol::Private::GetToolsetRegistry();
	if (!ToolsetRegistry)
	{
		OnComplete(UE::ModelContextProtocol::MakeErrorResult(TEXT("ToolsetRegistry not available")));
		return;
	}

	auto DescriptorResult = UE::ToolsetRegistry::FToolDescriptor::FromString(ToolName);
	if (!DescriptorResult.HasValue())
	{
		OnComplete(UE::ModelContextProtocol::MakeErrorResult(FString::Printf(TEXT("Invalid tool descriptor: %s"), *ToolName)));
		return;
	}

	FString ArgumentsJson = TEXT("{}");
	if (Params.IsValid())
	{
		ArgumentsJson.Reset();
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&ArgumentsJson);
		if (!FJsonSerializer::Serialize(Params.ToSharedRef(), Writer))
		{
			OnComplete(UE::ModelContextProtocol::MakeErrorResult(FString::Printf(TEXT("Failed to serialize arguments JSON for tool '%s'."), *ToolName)));
			return;
		}
	}

	UE_LOGF(LogModelContextProtocol, Log, "Running ToolsetRegistry tool: '%ls'", *ToolName);

	ToolsetRegistry->ExecuteTool(DescriptorResult.GetValue(), ArgumentsJson).Then([OnComplete](TFuture<TValueOrError<FString, FString>> Future)
	{
		TValueOrError<FString, FString> Result = Future.Get();

		if (Result.HasError())
		{
			OnComplete(UE::ModelContextProtocol::MakeErrorResult(Result.GetError()));
		}
		else
		{
			OnComplete(UE::ModelContextProtocol::MakeTextResult(Result.GetValue()));
		}
	});
}

int32 FToolsetRegistryToolAdapterManager::RegisterToolsFromSchema(const FString& ToolsetSchemaJson)
{
	IModelContextProtocolModule* const Module = IModelContextProtocolModule::Get();
	if (!Module)
	{
		return 0;
	}

	TSharedPtr<FJsonObject> ToolsetSchema;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ToolsetSchemaJson);
	if (!FJsonSerializer::Deserialize(Reader, ToolsetSchema) || !ToolsetSchema.IsValid())
	{
		return 0;
	}

	const TArray<TSharedPtr<FJsonValue>>* ToolsField = nullptr;
	if (!ToolsetSchema->TryGetArrayField(TEXT("tools"), ToolsField) || !ToolsField)
	{
		return 0;
	}

	int32 RegisteredCount = 0;
	for (const TSharedPtr<FJsonValue>& ToolValue : *ToolsField)
	{
		const TSharedPtr<FJsonObject>& ToolJsonObject = ToolValue->AsObject();
		if (!ToolJsonObject.IsValid())
		{
			continue;
		}

		FString ToolName;
		ToolJsonObject->TryGetStringField(TEXT("name"), ToolName);
		if (ToolName.IsEmpty())
		{
			continue;
		}

		FString ToolDescription;
		ToolJsonObject->TryGetStringField(TEXT("description"), ToolDescription);

		TSharedPtr<FJsonObject> InputSchema;
		const TSharedPtr<FJsonObject>* InputSchemaField = nullptr;
		if (ToolJsonObject->TryGetObjectField(TEXT("inputSchema"), InputSchemaField) && InputSchemaField)
		{
			InputSchema = *InputSchemaField;
		}

		TSharedRef<FToolsetRegistryToolAdapter> Adapter =
			MakeShared<FToolsetRegistryToolAdapter>(ToolName, ToolDescription, InputSchema);
		if (Module->AddTool(Adapter))
		{
			RegisteredAdapters.Add(Adapter);
			RegisteredCount++;
		}
	}

	return RegisteredCount;
}

void FToolsetRegistryToolAdapterManager::RegisterTools()
{
	DeregisterTools(false);

	UE::ToolsetRegistry::FToolsetRegistry* ToolsetRegistry = UE::ModelContextProtocol::Private::GetToolsetRegistry();
	if (!ToolsetRegistry)
	{
		return;
	}

	IModelContextProtocolModule* Module = IModelContextProtocolModule::Get();
	if (!Module)
	{
		return;
	}

	if (GetDefault<UModelContextProtocolSettings>()->bEnableToolSearch)
	{
		// Tool-search mode: register only the discovery and dispatch meta-tools; toolset tools are never registered as native MCP tools.
		auto RegisterMetaTool = [Module, this](const TSharedRef<IModelContextProtocolTool>& Tool)
		{
			if (Module->AddTool(Tool))
			{
				MetaTools.Add(Tool);
			}
		};

		RegisterMetaTool(MakeShared<FListToolsetsTool>(FListToolsetsDelegate([this]()
		{
			return GetToolsetCatalogText();
		})));

		RegisterMetaTool(MakeShared<FDescribeToolsetTool>(FDescribeToolsetDelegate([this](const FString& ToolsetName)
		{
			return GetToolsetSchemaText(ToolsetName);
		})));

		RegisterMetaTool(MakeShared<FCallTool>(FCallToolDelegate(
			[this](const FString& ToolsetName, const FString& ToolName, const TSharedPtr<FJsonObject>& Arguments, const FModelContextProtocolToolRequestId& RequestId, const IModelContextProtocolTool::FResultCallback& OnComplete)
		{
			DispatchToolCall(ToolsetName, ToolName, Arguments, RequestId, OnComplete);
		})));

		int32 ToolsetCount = 0;
		ToolsetRegistry->ForEachToolset([&ToolsetCount](const FString&, const UE::ToolsetRegistry::FToolset&)
		{
			ToolsetCount++;
		});

		UE_LOGF(LogModelContextProtocol, Log, "Tool search enabled: registered %d meta-tools (%d toolsets discoverable via list_toolsets)", MetaTools.Num(), ToolsetCount);
	}
	else
	{
		// Eager mode: register every toolset's tools as native MCP tools.
		ToolsetRegistry->ForEachToolset([this](const FString&, const UE::ToolsetRegistry::FToolset& Toolset)
		{
			RegisterToolsFromSchema(Toolset.GetJsonSchema());
		});

		UE_LOGF(LogModelContextProtocol, Log, "Tool search disabled: registered %d ToolsetRegistry tool adapters", RegisteredAdapters.Num());
	}
}

void FToolsetRegistryToolAdapterManager::DeregisterTools(bool bBroadcast)
{
	if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
	{
		for (const TSharedRef<FToolsetRegistryToolAdapter>& Adapter : RegisteredAdapters)
		{
			Module->RemoveTool(Adapter);
		}

		for (const TSharedRef<IModelContextProtocolTool>& Tool : MetaTools)
		{
			Module->RemoveTool(Tool);
		}
	}

	RegisteredAdapters.Reset();
	MetaTools.Reset();

	if (bBroadcast)
	{
		if (IModelContextProtocolModule* const Module = IModelContextProtocolModule::Get())
		{
			if (FModelContextProtocolServer* const Server = Module->GetServer())
			{
				Server->ScheduleToolsListChangedBroadcast();
			}
		}
	}
}

void FToolsetRegistryToolAdapterManager::DispatchToolCall(const FString& ToolsetName, const FString& ToolName, const TSharedPtr<FJsonObject>& Arguments, const FModelContextProtocolToolRequestId& RequestId, const IModelContextProtocolTool::FResultCallback& OnComplete) const
{
	if (!ToolsetName.IsEmpty())
	{
		UE::ToolsetRegistry::FToolsetRegistry* ToolsetRegistry = UE::ModelContextProtocol::Private::GetToolsetRegistry();
		if (!ToolsetRegistry)
		{
			OnComplete(UE::ModelContextProtocol::MakeErrorResult(TEXT("ToolsetRegistry not available")));
			return;
		}

		FString ArgumentsJson = TEXT("{}");
		if (Arguments.IsValid())
		{
			ArgumentsJson.Reset();
			TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&ArgumentsJson);
			if (!FJsonSerializer::Serialize(Arguments.ToSharedRef(), Writer))
			{
				OnComplete(UE::ModelContextProtocol::MakeErrorResult(TEXT("Failed to serialize arguments JSON for toolset dispatch.")));
				return;
			}
		}

		UE_LOGF(LogModelContextProtocol, Log, "Dispatching toolset tool: '%ls.%ls'", *ToolsetName, *ToolName);

		ToolsetRegistry->ExecuteTool(UE::ToolsetRegistry::FToolDescriptor{ToolsetName, ToolName}, ArgumentsJson).Then([OnComplete](TFuture<TValueOrError<FString, FString>> Future)
		{
			TValueOrError<FString, FString> Result = Future.Get();
			if (Result.HasError())
			{
				OnComplete(UE::ModelContextProtocol::MakeErrorResult(Result.GetError()));
			}
			else
			{
				OnComplete(UE::ModelContextProtocol::MakeTextResult(Result.GetValue()));
			}
		});
		return;
	}

	// Top-level self-dispatch through call_tool would recurse; reject explicitly. The toolset-qualified shape `(toolset_name=X, tool_name=call_tool)` is left to the toolset registry to reject as "tool not found", which preserves the freedom for a toolset to legitimately own a tool literally named `call_tool` should that ever happen.
	// `IModelContextProtocolModule::FindTool` is documented case-insensitive, so the guard matches its semantics: a mixed-case `Call_Tool` resolves to the meta-tool via FindTool and must therefore also be rejected here.
	if (ToolName.Equals(UE::ModelContextProtocol::CallToolName, ESearchCase::IgnoreCase))
	{
		OnComplete(UE::ModelContextProtocol::MakeErrorResult(TEXT("call_tool cannot dispatch to itself")));
		return;
	}

	IModelContextProtocolModule* Module = IModelContextProtocolModule::Get();
	if (!Module)
	{
		OnComplete(UE::ModelContextProtocol::MakeErrorResult(TEXT("MCP module not available")));
		return;
	}

	TSharedPtr<IModelContextProtocolTool> Tool = Module->FindTool(ToolName);
	if (!Tool.IsValid())
	{
		OnComplete(UE::ModelContextProtocol::MakeErrorResult(FString::Printf(TEXT("Tool '%s' not found"), *ToolName)));
		return;
	}

	UE_LOGF(LogModelContextProtocol, Log, "Dispatching top-level tool: '%ls'", *ToolName);

	Tool->RunAsync(RequestId, Arguments, OnComplete);
}

FString FToolsetRegistryToolAdapterManager::GetToolsetCatalogText() const
{
	UE::ToolsetRegistry::FToolsetRegistry* ToolsetRegistry = UE::ModelContextProtocol::Private::GetToolsetRegistry();
	if (!ToolsetRegistry)
	{
		return FString();
	}

	FString Result;
	ToolsetRegistry->ForEachToolset(
		[&Result](const FString& Name, const UE::ToolsetRegistry::FToolset& Toolset)
		{
			Result += FString::Printf(TEXT("- %s"), *Name);
			FString Description = Toolset.GetToolsetDescription();
			if (!Description.IsEmpty())
			{
				Result += FString::Printf(TEXT(": %s"), *Description);
			}
			Result += TEXT("\n");
		});
	return Result;
}

TValueOrError<FString, FString> FToolsetRegistryToolAdapterManager::GetToolsetSchemaText(
	const FString& ToolsetName) const
{
	UE::ToolsetRegistry::FToolsetRegistry* ToolsetRegistry = UE::ModelContextProtocol::Private::GetToolsetRegistry();
	if (!ToolsetRegistry)
	{
		return MakeError(TEXT("ToolsetRegistry not available."));
	}

	TSharedPtr<UE::ToolsetRegistry::FToolset> Toolset = ToolsetRegistry->Find(ToolsetName);
	if (!Toolset.IsValid())
	{
		TArray<FString> Names;
		ToolsetRegistry->ForEachToolset(
			[&Names](const FString& Name, const UE::ToolsetRegistry::FToolset&)
			{
				Names.Add(Name);
			});
		return MakeError(FString::Printf(
			TEXT("Toolset '%s' not found. Available toolsets: %s"),
			*ToolsetName, *FString::Join(Names, TEXT(", "))));
	}

	return MakeValue(Toolset->GetJsonSchema());
}
