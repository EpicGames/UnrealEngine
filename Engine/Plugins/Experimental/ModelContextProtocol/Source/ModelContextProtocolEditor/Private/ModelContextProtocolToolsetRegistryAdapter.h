// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IModelContextProtocolTool.h"

namespace UE::ToolsetRegistry { struct FToolDescriptor; }

/**
 * Adapts a ToolsetRegistry tool to the IModelContextProtocolTool interface.
 * Created by the Editor module to bridge ToolsetRegistry tools into the MCP server.
 */
struct FToolsetRegistryToolAdapter : IModelContextProtocolTool
{
	FToolsetRegistryToolAdapter(const FString& InToolName, const FString& InDescription, TSharedPtr<FJsonObject> InInputSchema);

	virtual FString GetName() const override { return ToolName; }
	virtual FString GetDescription() const override { return Description; }
	virtual TSharedPtr<FJsonObject> GetInputJsonSchema() const override { return InputSchema; }
	virtual void RunAsync(const FModelContextProtocolToolRequestId& RequestId, const TSharedPtr<FJsonObject>& Params, const FResultCallback& OnComplete) override;

private:
	FString ToolName;
	FString Description;
	TSharedPtr<FJsonObject> InputSchema;
};

/**
 * Registers and manages ToolsetRegistry tool adapters with the MCP module.
 * In eager mode (bEnableToolSearch = false), every toolset tool is registered as a native MCP tool at startup.
 * In tool-search mode (bEnableToolSearch = true), only list_toolsets, describe_toolset, and call_tool are registered; toolset tools are dispatched through call_tool without being registered.
 */
struct FToolsetRegistryToolAdapterManager
{
	~FToolsetRegistryToolAdapterManager()
	{
		DeregisterTools(false);
	}

	/**
	 * Enumerates ToolsetRegistry tools and registers them with the MCP module.
	 * In eager mode, registers all tool adapters directly. In tool-search mode, registers list_toolsets, describe_toolset, and call_tool for on-demand discovery and dispatch.
	 */
	void RegisterTools();

	/** Removes all previously registered adapter tools. Optionally broadcasts tools/list_changed. */
	void DeregisterTools(bool bBroadcast = true);

private:
	/** Register all tools from a toolset's JSON schema string. Returns the number of tools registered. */
	int32 RegisterToolsFromSchema(const FString& ToolsetSchemaJson);

	/** Returns a compact text listing of all toolsets (name + description). */
	FString GetToolsetCatalogText() const;

	/** Returns the full JSON schema for a named toolset, or an error. */
	TValueOrError<FString, FString> GetToolsetSchemaText(const FString& ToolsetName) const;

	/** Dispatches a tool call through ToolsetRegistry (when ToolsetName is non-empty) or through the MCP module's top-level tool table (otherwise). Result is delivered via OnComplete. */
	void DispatchToolCall(const FString& ToolsetName, const FString& ToolName, const TSharedPtr<FJsonObject>& Arguments, const FModelContextProtocolToolRequestId& RequestId, const IModelContextProtocolTool::FResultCallback& OnComplete) const;

	TArray<TSharedRef<FToolsetRegistryToolAdapter>> RegisteredAdapters;
	TArray<TSharedRef<IModelContextProtocolTool>> MetaTools;
};
