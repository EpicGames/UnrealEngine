// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Script/AssetDefinition_Blueprint.h"

#include "ModelContextProtocolAssetDefinitions.generated.h"

#define UE_API MODELCONTEXTPROTOCOLEDITOR_API

UCLASS()
class UAssetDefinition_ModelContextProtocolToolLibraryBlueprint : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
public:
	//~ UAssetDefinition_Blueprint Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	//~ UAssetDefinition_Blueprint End
};

UCLASS()
class UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
public:
	//~ UAssetDefinition_Blueprint Begin
	virtual FText GetAssetDisplayName() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	//~ UAssetDefinition_Blueprint End
};

#undef UE_API
