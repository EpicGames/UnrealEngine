// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocolToolsetRegistryAdapter.h"

#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/DelegateHandle.h"

class FModelContextProtocolEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void SetupEditorIntegration();

	FToolsetRegistryToolAdapterManager ToolsetRegistryAdapterManager;
	UE::ToolsetRegistry::FDelegateHandleRaii PostEngineInitHandle;
	FDelegateHandle OnRefreshToolsHandle;
	FDelegateHandle OnToolsetRegisteredHandle;
};
