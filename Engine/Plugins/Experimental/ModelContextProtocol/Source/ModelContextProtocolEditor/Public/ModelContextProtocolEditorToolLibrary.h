// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUtilityBlueprint.h"
#include "ModelContextProtocolToolLibrary.h"

#include "ModelContextProtocolEditorToolLibrary.generated.h"

#define UE_API MODELCONTEXTPROTOCOL_API

/**
 * Automatically registers an FModelContextProtocolLibraryTool for each public UFUNCTION on module load, using cached / cooked meta-data.
 * 
 * UBlueprintFunctionLibrary specialization allows caching and cooking otherwise-editor-only UFUNCTION descriptions and parameter meta-data.
 *
 * Can be subclassed in either C++ or Blueprints.
 *
 * For Blueprints:	Create via Content Browser -> Add -> MCP Tool Library, then simply define public functions with a doxygen-style
 *					function tooltip.   
 */
UCLASS(Abstract, MinimalAPI)
class UModelContextProtocolEditorToolLibrary : public UModelContextProtocolToolLibrary
{
	GENERATED_BODY()
};
/*
	Note: Like UEditorFunctionLibrary, there is no reason to extend this in c++, it is used as a sentinel type for blueprints that want to provide
	static reusable functions for editing logic. In c++ you can just extend UBlueprintFunctionLibrary as normal and put your class in an uncooked
	or editor only module or WITH_EDITOR block.
*/


/**
 * Editor-specific UModelContextProtocolToolLibrary function library editor utility blueprint, allowing Editor-only BP node use in MCP tool functions.
 * @see UModelContextProtocolToolLibraryBlueprint
 */ 
UCLASS(MinimalAPI)
class UModelContextProtocolEditorToolLibraryBlueprint : public UEditorUtilityBlueprint
{
	GENERATED_BODY()
public:
	UModelContextProtocolEditorToolLibraryBlueprint();

	//~ Begin UBlueprint API
#if WITH_EDITOR
	virtual bool AlwaysCompileOnLoad() const override;
#endif // WITH_EDITOR
	//~ End UBlueprint API
};

#undef UE_API
