// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/BlueprintFactory.h"

#include "ModelContextProtocolToolLibraryFactory.generated.h"

#define UE_API MODELCONTEXTPROTOCOLEDITOR_API

UCLASS(MinimalAPI)
class UModelContextProtocolToolLibraryFactory : public UBlueprintFactory
{
	GENERATED_BODY()
public:
	UModelContextProtocolToolLibraryFactory(const FObjectInitializer& ObjectInitializer);

	// UFactory interface
	virtual FText GetDisplayName() const override;
	virtual FName GetNewAssetThumbnailOverride() const override;
	virtual FText GetToolTip() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual bool ConfigureProperties() override;
	virtual FString GetDefaultNewAssetName() const override;
	// End of UFactory interface
};

/**
 * Editor-specific UModelContextProtocolToolLibrary factory which produces a UModelContextProtocolEditorToolLibraryBlueprint editor utility blueprint,
 * allowing editor-specific BP node usage in tool function implementations.
 */
UCLASS(MinimalAPI)
class UModelContextProtocolEditorToolLibraryFactory : public UModelContextProtocolToolLibraryFactory
{
	GENERATED_BODY()
public:
	UModelContextProtocolEditorToolLibraryFactory(const FObjectInitializer& ObjectInitializer);

	// UFactory interface
	virtual FText GetDisplayName() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn, FName CallingContext) override;
	virtual FString GetDefaultNewAssetName() const override;
	// End of UFactory interface
};

#undef UE_API
