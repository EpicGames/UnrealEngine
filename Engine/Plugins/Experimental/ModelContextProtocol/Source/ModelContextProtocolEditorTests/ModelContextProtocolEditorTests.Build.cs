// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelContextProtocolEditorTests : ModuleRules
{
	public ModelContextProtocolEditorTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// Include paths for shared test utilities and mocks from ModelContextProtocolTests
		PrivateIncludePaths.Add(System.IO.Path.Combine(PluginDirectory, "Source", "ModelContextProtocolTests", "Private"));

		// Access to ModelContextProtocolEditor private headers so tests can reference sentinel name constants and exercise UCommandlet testable entry points.
		PrivateIncludePaths.Add(System.IO.Path.Combine(PluginDirectory, "Source", "ModelContextProtocolEditor", "Private"));

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Json",
				"JsonUtilities",
				"HTTPServer",
				"HTTP",
				"ModelContextProtocol",
				"ModelContextProtocolEngine",
				"ModelContextProtocolEditor",
				"ModelContextProtocolTests"
			});

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
					"Slate",
					"SlateCore",
					"AssetDefinition",
					"Blutility",
					"ToolsetRegistry"
				});
		}
	}
}
