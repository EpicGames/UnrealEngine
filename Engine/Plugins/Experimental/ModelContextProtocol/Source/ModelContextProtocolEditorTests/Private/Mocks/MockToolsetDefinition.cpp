// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mocks/MockToolsetDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MockToolsetDefinition)

FString UMockToolsetDefinition::Greet(const FString& Name)
{
	return FString::Printf(TEXT("Hello, %s!"), *Name);
}

int32 UMockToolsetDefinition::Add(int32 A, int32 B)
{
	return A + B;
}
