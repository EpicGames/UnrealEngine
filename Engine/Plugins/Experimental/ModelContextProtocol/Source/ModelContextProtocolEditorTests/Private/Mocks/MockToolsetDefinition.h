// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolsetRegistry/ToolsetDefinition.h"

#include "MockToolsetDefinition.generated.h"

/**
 * Mock UToolsetDefinition subclass for testing ToolsetRegistry integration with MCP.
 */
UCLASS()
class UMockToolsetDefinition : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/** Greets a person by name */
	UFUNCTION(meta = (AICallable))
	static FString Greet(const FString& Name);

	/** Adds two numbers together */
	UFUNCTION(meta = (AICallable))
	static int32 Add(int32 A, int32 B);
};
