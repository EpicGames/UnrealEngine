// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolAssetDefinitions.h"
#include "ModelContextProtocolToolLibrary.h"
#include "ModelContextProtocolEditorToolLibrary.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolAssetDefinitionsTests, "AI.ModelContextProtocol.Editor.AssetDefinitions", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FModelContextProtocolAssetDefinitionsTests)

void FModelContextProtocolAssetDefinitionsTests::Define()
{
	Describe("UAssetDefinition_ModelContextProtocolToolLibraryBlueprint", [this]()
	{
		It("should return the correct asset class", [this]()
		{
			UAssetDefinition_ModelContextProtocolToolLibraryBlueprint* AssetDef = NewObject<UAssetDefinition_ModelContextProtocolToolLibraryBlueprint>();
			TSoftClassPtr<UObject> AssetClass = AssetDef->GetAssetClass();
			TestEqual("Asset class should be UModelContextProtocolToolLibraryBlueprint",
				AssetClass, TSoftClassPtr<UObject>(UModelContextProtocolToolLibraryBlueprint::StaticClass()));
		});

		It("should return non-empty display name", [this]()
		{
			UAssetDefinition_ModelContextProtocolToolLibraryBlueprint* AssetDef = NewObject<UAssetDefinition_ModelContextProtocolToolLibraryBlueprint>();
			FText DisplayName = AssetDef->GetAssetDisplayName();
			TestFalse("Display name should not be empty", DisplayName.IsEmpty());
		});

		It("should return non-empty asset categories", [this]()
		{
			UAssetDefinition_ModelContextProtocolToolLibraryBlueprint* AssetDef = NewObject<UAssetDefinition_ModelContextProtocolToolLibraryBlueprint>();
			TConstArrayView<FAssetCategoryPath> Categories = AssetDef->GetAssetCategories();
			TestTrue("Should have at least one category", Categories.Num() > 0);
		});
	});

	Describe("UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint", [this]()
	{
		It("should return the correct asset class", [this]()
		{
			UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint* AssetDef = NewObject<UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint>();
			TSoftClassPtr<UObject> AssetClass = AssetDef->GetAssetClass();
			TestEqual("Asset class should be UModelContextProtocolEditorToolLibraryBlueprint",
				AssetClass, TSoftClassPtr<UObject>(UModelContextProtocolEditorToolLibraryBlueprint::StaticClass()));
		});

		It("should return non-empty display name", [this]()
		{
			UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint* AssetDef = NewObject<UAssetDefinition_ModelContextProtocolEditorToolLibraryBlueprint>();
			FText DisplayName = AssetDef->GetAssetDisplayName();
			TestFalse("Display name should not be empty", DisplayName.IsEmpty());
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
