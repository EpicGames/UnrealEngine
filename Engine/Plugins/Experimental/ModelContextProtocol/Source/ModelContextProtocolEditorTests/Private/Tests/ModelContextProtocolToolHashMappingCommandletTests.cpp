// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolAnalytics.h"
#include "ModelContextProtocolToolHashMappingCommandlet.h"

#include "Containers/Set.h"
#include "Containers/SortedMap.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolToolHashMappingCommandletTests, "AI.ModelContextProtocol.ToolHashMappingCommandlet", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FModelContextProtocolToolHashMappingCommandletTests)

void FModelContextProtocolToolHashMappingCommandletTests::Define()
{
	using namespace UE::ModelContextProtocol::Analytics;

	Describe("BuildHashMaps", [this]()
	{
		It("should hash every tool name under its blake3 identifier", [this]()
		{
			const TSet<FString> Names = { TEXT("Alpha.One"), TEXT("Alpha.Two"), TEXT("Beta.Nested.Three") };

			TSortedMap<FString, FString> Tools;
			TSortedMap<FString, FString> Toolsets;
			UModelContextProtocolToolHashMappingCommandlet::BuildHashMaps(Names, Tools, Toolsets);

			TestEqual(TEXT("tool count"), Tools.Num(), 3);
			for (const FString& Name : Names)
			{
				const FString* Found = Tools.Find(HashToolIdentifier(Name));
				if (TestNotNull(TEXT("tool hash key present"), Found))
				{
					TestEqual(TEXT("tool hash maps to name"), *Found, Name);
				}
			}
		});

		It("should split toolset prefixes at the last dot", [this]()
		{
			// Matches FToolDescriptor::FromString (and ParseToolsetName in ModelContextProtocolAnalytics.cpp): 'Beta.Nested.Three' -> 'Beta.Nested'.
			const TSet<FString> Names = { TEXT("Alpha.One"), TEXT("Beta.Nested.Three") };

			TSortedMap<FString, FString> Tools;
			TSortedMap<FString, FString> Toolsets;
			UModelContextProtocolToolHashMappingCommandlet::BuildHashMaps(Names, Tools, Toolsets);

			const FString* AlphaEntry = Toolsets.Find(HashToolIdentifier(TEXT("Alpha")));
			if (TestNotNull(TEXT("Alpha toolset present"), AlphaEntry))
			{
				TestEqual(TEXT("Alpha toolset prefix"), *AlphaEntry, TEXT("Alpha"));
			}

			const FString* BetaNestedEntry = Toolsets.Find(HashToolIdentifier(TEXT("Beta.Nested")));
			if (TestNotNull(TEXT("Beta.Nested toolset present"), BetaNestedEntry))
			{
				TestEqual(TEXT("Beta.Nested toolset prefix"), *BetaNestedEntry, TEXT("Beta.Nested"));
			}
		});

		It("should emit an empty-prefix toolset entry when any top-level tool is present", [this]()
		{
			const TSet<FString> Names = { TEXT("top_level_tool"), TEXT("Alpha.One") };

			TSortedMap<FString, FString> Tools;
			TSortedMap<FString, FString> Toolsets;
			UModelContextProtocolToolHashMappingCommandlet::BuildHashMaps(Names, Tools, Toolsets);

			const FString* EmptyEntry = Toolsets.Find(HashToolIdentifier(FString()));
			if (TestNotNull(TEXT("empty-prefix toolset present"), EmptyEntry))
			{
				TestEqual(TEXT("empty-prefix value is the empty string"), *EmptyEntry, FString());
			}
		});

		It("should not emit the empty-prefix entry when every tool is dotted", [this]()
		{
			const TSet<FString> Names = { TEXT("Alpha.One"), TEXT("Beta.Two") };

			TSortedMap<FString, FString> Tools;
			TSortedMap<FString, FString> Toolsets;
			UModelContextProtocolToolHashMappingCommandlet::BuildHashMaps(Names, Tools, Toolsets);

			TestNull(TEXT("empty-prefix toolset absent"), Toolsets.Find(HashToolIdentifier(FString())));
		});

		It("should dedupe the toolset map when multiple tools share a prefix", [this]()
		{
			const TSet<FString> Names = { TEXT("Alpha.One"), TEXT("Alpha.Two"), TEXT("Alpha.Three") };

			TSortedMap<FString, FString> Tools;
			TSortedMap<FString, FString> Toolsets;
			UModelContextProtocolToolHashMappingCommandlet::BuildHashMaps(Names, Tools, Toolsets);

			TestEqual(TEXT("single Alpha toolset entry"), Toolsets.Num(), 1);
		});

		It("should handle an empty input set without producing any entries", [this]()
		{
			const TSet<FString> Names;

			TSortedMap<FString, FString> Tools;
			TSortedMap<FString, FString> Toolsets;
			UModelContextProtocolToolHashMappingCommandlet::BuildHashMaps(Names, Tools, Toolsets);

			TestEqual(TEXT("no tools"), Tools.Num(), 0);
			TestEqual(TEXT("no toolsets"), Toolsets.Num(), 0);
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
