// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolLibraryFactory.h"
#include "ModelContextProtocolToolLibrary.h"
#include "ModelContextProtocolEditorToolLibrary.h"
#include "Misc/AutomationTest.h"
#include "UObject/Package.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolToolLibraryFactoryTests, "AI.ModelContextProtocol.Editor.Factory", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	TArray<UObject*> CreatedObjects;
END_DEFINE_SPEC(FModelContextProtocolToolLibraryFactoryTests)

void FModelContextProtocolToolLibraryFactoryTests::Define()
{
	AfterEach([this]()
	{
		for (UObject* Object : CreatedObjects)
		{
			if (Object)
			{
				Object->Rename(nullptr, static_cast<UObject*>(GetTransientPackage()), REN_DontCreateRedirectors | REN_NonTransactional);
				Object->MarkAsGarbage();
			}
		}
		CreatedObjects.Empty();
	});

	Describe("UModelContextProtocolToolLibraryFactory", [this]()
	{
		It("should produce a valid UModelContextProtocolToolLibraryBlueprint via FactoryCreateNew", [this]()
		{
			UModelContextProtocolToolLibraryFactory* Factory = NewObject<UModelContextProtocolToolLibraryFactory>();
			UPackage* TransientPackage = GetTransientPackage();
			UObject* CreatedObject = Factory->FactoryCreateNew(
				UModelContextProtocolToolLibraryBlueprint::StaticClass(),
				TransientPackage,
				MakeUniqueObjectName(static_cast<UObject*>(TransientPackage), UModelContextProtocolToolLibraryBlueprint::StaticClass(), FName(TEXT("TestMCPToolLibrary"))),
				RF_Transient,
				nullptr,
				GWarn,
				NAME_None);
			TestNotNull("FactoryCreateNew should produce an object", CreatedObject);
			if (CreatedObject)
			{
				CreatedObjects.Add(CreatedObject);
				TestTrue("Should be a UModelContextProtocolToolLibraryBlueprint",
					CreatedObject->IsA<UModelContextProtocolToolLibraryBlueprint>());
			}
		});

		It("should return expected display name", [this]()
		{
			UModelContextProtocolToolLibraryFactory* Factory = NewObject<UModelContextProtocolToolLibraryFactory>();
			FText DisplayName = Factory->GetDisplayName();
			TestFalse("Display name should not be empty", DisplayName.IsEmpty());
		});

		It("should return non-zero menu categories", [this]()
		{
			UModelContextProtocolToolLibraryFactory* Factory = NewObject<UModelContextProtocolToolLibraryFactory>();
			uint32 Categories = Factory->GetMenuCategories();
			TestTrue("Menu categories should be non-zero", Categories != 0);
		});
	});

	Describe("UModelContextProtocolEditorToolLibraryFactory", [this]()
	{
		It("should produce a valid UModelContextProtocolEditorToolLibraryBlueprint via FactoryCreateNew", [this]()
		{
			UModelContextProtocolEditorToolLibraryFactory* Factory = NewObject<UModelContextProtocolEditorToolLibraryFactory>();
			UPackage* TransientPackage = GetTransientPackage();
			UObject* CreatedObject = Factory->FactoryCreateNew(
				UModelContextProtocolEditorToolLibraryBlueprint::StaticClass(),
				TransientPackage,
				MakeUniqueObjectName(static_cast<UObject*>(TransientPackage), UModelContextProtocolEditorToolLibraryBlueprint::StaticClass(), FName(TEXT("TestMCPEditorToolLibrary"))),
				RF_Transient,
				nullptr,
				GWarn,
				NAME_None);
			TestNotNull("FactoryCreateNew should produce an object", CreatedObject);
			if (CreatedObject)
			{
				CreatedObjects.Add(CreatedObject);
				TestTrue("Should be a UModelContextProtocolEditorToolLibraryBlueprint",
					CreatedObject->IsA<UModelContextProtocolEditorToolLibraryBlueprint>());
			}
		});

		It("should return expected display name", [this]()
		{
			UModelContextProtocolEditorToolLibraryFactory* Factory = NewObject<UModelContextProtocolEditorToolLibraryFactory>();
			FText DisplayName = Factory->GetDisplayName();
			TestFalse("Display name should not be empty", DisplayName.IsEmpty());
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
