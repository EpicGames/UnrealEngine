// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolUtils.h"
#include "ModelContextProtocolToolResults.h"

#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "GameFramework/Actor.h"
#include "Sound/SoundWave.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolToolUtilsTests, "AI.ModelContextProtocol.ToolUtils", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FModelContextProtocolToolUtilsTests)

void FModelContextProtocolToolUtilsTests::Define()
{
	using namespace UE::ModelContextProtocol;

	Describe("GetToolResultType", [this]()
	{
		It("should return None for nullptr property", [this]()
		{
			TSharedPtr<FJsonObject> OutSchema;
			EModelContextProtocolToolResultType Result = GetToolResultType(nullptr, nullptr, OutSchema);
			TestEqual("Should return None", Result, EModelContextProtocolToolResultType::None);
		});

		It("should return Text for FStrProperty", [this]()
		{
			// Find an FStrProperty from a known UStruct (AActor has many string properties)
			FProperty* StringProperty = nullptr;
			if (TFieldIterator<FStrProperty> It(AActor::StaticClass()); It)
			{
				StringProperty = *It;
			}
			if (!TestNotNull("Should find an FStrProperty on AActor", StringProperty)) { return; }

			TSharedPtr<FJsonObject> OutSchema;
			EModelContextProtocolToolResultType Result = GetToolResultType(StringProperty, nullptr, OutSchema);
			TestEqual("Should return Text for FStrProperty", Result, EModelContextProtocolToolResultType::Text);
		});

		It("should return Image for UTexture2D object property", [this]()
		{
			// Search all loaded classes for an FObjectProperty with PropertyClass == UTexture2D
			FObjectProperty* TextureProperty = nullptr;
			for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
			{
				for (TFieldIterator<FObjectProperty> PropIt(*ClassIt, EFieldIteratorFlags::ExcludeSuper); PropIt; ++PropIt)
				{
					if (PropIt->PropertyClass == UTexture2D::StaticClass())
					{
						TextureProperty = *PropIt;
						break;
					}
				}
				if (TextureProperty) { break; }
			}

			if (!TestNotNull("Should find a UTexture2D object property on any loaded class", TextureProperty)) { return; }

			TSharedPtr<FJsonObject> OutSchema;
			EModelContextProtocolToolResultType Result = GetToolResultType(TextureProperty, nullptr, OutSchema);
			TestEqual("Should return Image for UTexture2D property", Result, EModelContextProtocolToolResultType::Image);
		});

		It("should return StructuredContent for struct properties that are not text/image/audio", [this]()
		{
			// Find a property that is not string, not UTexture2D, not USoundWave
			FProperty* StructProperty = nullptr;
			if (TFieldIterator<FStructProperty> It(AActor::StaticClass()); It)
			{
				StructProperty = *It;
			}
			if (!TestNotNull("Should find a struct property on AActor", StructProperty)) { return; }

			TSharedPtr<FJsonObject> OutSchema;
			EModelContextProtocolToolResultType Result = GetToolResultType(StructProperty, nullptr, OutSchema);
			TestEqual("Should return StructuredContent for struct property", Result, EModelContextProtocolToolResultType::StructuredContent);
		});

		It("should return Text for FSoftObjectProperty", [this]()
		{
			FProperty* SoftObjectProperty = nullptr;
			for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
			{
				if (TFieldIterator<FSoftObjectProperty> PropIt(*ClassIt, EFieldIteratorFlags::ExcludeSuper); PropIt)
				{
					SoftObjectProperty = *PropIt;
				}
				if (SoftObjectProperty) { break; }
			}

			if (!TestNotNull("Should find a FSoftObjectProperty on any loaded class", SoftObjectProperty)) { return; }

			TSharedPtr<FJsonObject> OutSchema;
			EModelContextProtocolToolResultType Result = GetToolResultType(SoftObjectProperty, nullptr, OutSchema);
			TestEqual("Should return Text for FSoftObjectProperty", Result, EModelContextProtocolToolResultType::Text);
		});

		It("should populate OutSchema for StructuredContent results", [this]()
		{
			FProperty* StructProperty = nullptr;
			if (TFieldIterator<FStructProperty> It(AActor::StaticClass()); It)
			{
				StructProperty = *It;
			}
			if (!TestNotNull("Should find a struct property", StructProperty)) { return; }

			TSharedPtr<FJsonObject> OutSchema;
			GetToolResultType(StructProperty, nullptr, OutSchema);
			TestTrue("OutSchema should be populated for StructuredContent", OutSchema.IsValid());
		});

		It("should not populate OutSchema for Text results", [this]()
		{
			FProperty* StringProperty = nullptr;
			if (TFieldIterator<FStrProperty> It(AActor::StaticClass()); It)
			{
				StringProperty = *It;
			}
			if (!TestNotNull("Should find a string property", StringProperty)) { return; }

			TSharedPtr<FJsonObject> OutSchema;
			GetToolResultType(StringProperty, nullptr, OutSchema);
			TestFalse("OutSchema should not be populated for Text", OutSchema.IsValid());
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
