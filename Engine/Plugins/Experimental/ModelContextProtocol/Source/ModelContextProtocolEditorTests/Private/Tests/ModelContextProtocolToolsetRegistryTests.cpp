// Copyright Epic Games, Inc. All Rights Reserved.

#include "IModelContextProtocolModule.h"
#include "ModelContextProtocol.h"
#include "ModelContextProtocolTestUtilities.h"
#include "Mocks/MockModelContextProtocolTool.h"

#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "Misc/AutomationTest.h"

#if WITH_EDITOR
#include "Editor.h"
#include "ModelContextProtocolToolSearch.h"
#include "ModelContextProtocolSettings.h"
#include "ToolsetRegistry/ToolsetRegistry.h"
#include "ToolsetRegistry/ToolsetRegistrySubsystem.h"
#include "ToolsetRegistry/UToolsetRegistry.h"
#include "Mocks/MockToolsetDefinition.h"
#endif

#if WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR

// Qualified tool names follow the pattern: {ModuleName}.{ClassName}.{FunctionName}
static const FString MockToolsetName = TEXT("ModelContextProtocolEditorTests.MockToolsetDefinition");
static const FString GreetToolName = MockToolsetName + TEXT(".Greet");
static const FString AddToolName = MockToolsetName + TEXT(".Add");
static const FString GreetBareName = TEXT("Greet");
static const FString ListToolsetsToolName = UE::ModelContextProtocol::ListToolsetsName;
static const FString DescribeToolsetToolName = UE::ModelContextProtocol::DescribeToolsetName;
static const FString CallToolToolName = UE::ModelContextProtocol::CallToolName;

BEGIN_DEFINE_SPEC(FModelContextProtocolToolsetRegistryTests, "AI.ModelContextProtocol.ToolsetRegistry", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	FString SessionId;
	bool bSavedEnableToolSearch = false;

	void SendJsonRpcRequest(const TSharedRef<FJsonObject>& JsonRpcBody, const TFunction<void(FHttpResponsePtr, bool)>& OnComplete, const FString& InSessionId = FString(), TOptional<float> ActivityTimeout = {})
	{
		FHttpModule& HttpModule = FHttpModule::Get();
		TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
		HttpRequest->SetURL(UE::ModelContextProtocol::Tests::GetTestBaseUrl());
		HttpRequest->SetVerb(TEXT("POST"));
		HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
		if (!InSessionId.IsEmpty())
		{
			HttpRequest->SetHeader(TEXT("Mcp-Session-Id"), InSessionId);
		}
		if (ActivityTimeout.IsSet())
		{
			HttpRequest->SetActivityTimeout(ActivityTimeout.GetValue());
		}
		HttpRequest->SetContentAsString(UE::ModelContextProtocol::Tests::JsonObjectToString(JsonRpcBody));
		HttpRequest->OnProcessRequestComplete().BindLambda([OnComplete](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnectedSuccessfully)
		{
			OnComplete(Response, bConnectedSuccessfully);
		});
		HttpRequest->ProcessRequest();
	}

	TSharedPtr<FJsonObject> ParseJsonResponse(FHttpResponsePtr Response)
	{
		if (!Response.IsValid())
		{
			return nullptr;
		}
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
		FJsonSerializer::Deserialize(Reader, JsonObject);
		return JsonObject;
	}

	void InitializeSession(const FDoneDelegate& Done)
	{
		using namespace UE::ModelContextProtocol::Tests;

		TSharedRef<FJsonObject> InitParams = MakeShared<FJsonObject>();
		InitParams->SetStringField(TEXT("protocolVersion"), UE::ModelContextProtocol::ProtocolVersion);
		TSharedRef<FJsonObject> ClientInfo = MakeShared<FJsonObject>();
		ClientInfo->SetStringField(TEXT("name"), TEXT("TestClient"));
		ClientInfo->SetStringField(TEXT("version"), TEXT("1.0"));
		InitParams->SetObjectField(TEXT("clientInfo"), ClientInfo);
		InitParams->SetObjectField(TEXT("capabilities"), MakeShared<FJsonObject>());

		TSharedRef<FJsonObject> InitRequest = MakeJsonRpcRequest(TEXT("initialize"), MakeShared<FJsonValueNumber>(1), InitParams);
		SendJsonRpcRequest(InitRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
		{
			if (Response.IsValid())
			{
				SessionId = Response->GetHeader(TEXT("Mcp-Session-Id"));

				TSharedRef<FJsonObject> InitializedNotification = UE::ModelContextProtocol::Tests::MakeJsonRpcNotification(TEXT("notifications/initialized"));
				SendJsonRpcRequest(InitializedNotification, [Done](FHttpResponsePtr, bool)
				{
					Done.Execute();
				}, SessionId);
			}
			else
			{
				Done.Execute();
			}
		});
	}

	void CleanupSession(const FDoneDelegate& Done)
	{
		if (SessionId.IsEmpty())
		{
			Done.Execute();
			return;
		}

		FHttpModule& HttpModule = FHttpModule::Get();
		TSharedRef<IHttpRequest> DeleteRequest = HttpModule.CreateRequest();
		DeleteRequest->SetURL(UE::ModelContextProtocol::Tests::GetTestBaseUrl());
		DeleteRequest->SetVerb(TEXT("DELETE"));
		DeleteRequest->SetHeader(TEXT("Mcp-Session-Id"), SessionId);
		DeleteRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr, bool)
		{
			SessionId.Reset();
			Done.Execute();
		});
		DeleteRequest->ProcessRequest();
	}

	// Builds a tools/call params object that dispatches a toolset tool through call_tool.
	TSharedRef<FJsonObject> MakeExecuteToolCallParams(const FString& ToolsetName, const FString& BareToolName, const TSharedPtr<FJsonObject>& ToolArguments)
	{
		TSharedRef<FJsonObject> ExecuteArgs = MakeShared<FJsonObject>();
		if (!ToolsetName.IsEmpty())
		{
			ExecuteArgs->SetStringField(TEXT("toolset_name"), ToolsetName);
		}
		ExecuteArgs->SetStringField(TEXT("tool_name"), BareToolName);
		if (ToolArguments.IsValid())
		{
			ExecuteArgs->SetObjectField(TEXT("arguments"), ToolArguments.ToSharedRef());
		}

		TSharedRef<FJsonObject> CallParams = MakeShared<FJsonObject>();
		CallParams->SetStringField(TEXT("name"), CallToolToolName);
		CallParams->SetObjectField(TEXT("arguments"), ExecuteArgs);
		return CallParams;
	}

	void ConfigureMode(bool bEnableToolSearch)
	{
		UModelContextProtocolSettings* Settings = GetMutableDefault<UModelContextProtocolSettings>();
		bSavedEnableToolSearch = Settings->bEnableToolSearch;
		Settings->bEnableToolSearch = bEnableToolSearch;
		UToolsetRegistry::RegisterToolsetClass(UMockToolsetDefinition::StaticClass());
		IModelContextProtocolModule::GetChecked().RefreshTools();
	}

	void RestoreMode()
	{
		GetMutableDefault<UModelContextProtocolSettings>()->bEnableToolSearch = bSavedEnableToolSearch;
		UToolsetRegistry::UnregisterToolsetClass(UMockToolsetDefinition::StaticClass());
		IModelContextProtocolModule::GetChecked().RefreshTools();
	}
END_DEFINE_SPEC(FModelContextProtocolToolsetRegistryTests)

void FModelContextProtocolToolsetRegistryTests::Define()
{
	using namespace UE::ModelContextProtocol::Tests;

	// ---- Tool search disabled: every toolset tool is registered as a native MCP tool at startup. ----
	Describe("With tool search disabled", [this]()
	{
		BeforeEach([this]()
		{
			SessionId.Reset();
			ConfigureMode(/*bEnableToolSearch=*/false);
		});

		AfterEach([this]()
		{
			RestoreMode();
		});

		LatentIt("should include ToolsetRegistry tools in tools/list", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(2));
				SendJsonRpcRequest(ListRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ToolsArray;
								if (TestTrue("Should have tools array", (*ResultObject)->TryGetArrayField(TEXT("tools"), ToolsArray)))
								{
									bool bFoundGreet = false;
									bool bFoundAdd = false;
									for (const TSharedPtr<FJsonValue>& ToolValue : *ToolsArray)
									{
										const TSharedPtr<FJsonObject>* ToolObject;
										if (ToolValue->TryGetObject(ToolObject))
										{
											FString ToolName;
											(*ToolObject)->TryGetStringField(TEXT("name"), ToolName);
											if (ToolName == GreetToolName)
											{
												bFoundGreet = true;
												FString ToolDescription;
												(*ToolObject)->TryGetStringField(TEXT("description"), ToolDescription);
												TestFalse("Greet tool should have a description", ToolDescription.IsEmpty());
												const TSharedPtr<FJsonObject>* InputSchemaField;
												TestTrue("Greet tool should have inputSchema", (*ToolObject)->TryGetObjectField(TEXT("inputSchema"), InputSchemaField));
											}
											else if (ToolName == AddToolName)
											{
												bFoundAdd = true;
												const TSharedPtr<FJsonObject>* InputSchemaField;
												TestTrue("Add tool should have inputSchema", (*ToolObject)->TryGetObjectField(TEXT("inputSchema"), InputSchemaField));
											}
										}
									}
									TestTrue("Should find Greet tool from ToolsetRegistry", bFoundGreet);
									TestTrue("Should find Add tool from ToolsetRegistry", bFoundAdd);
								}
							}
						}
					}
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should execute ToolsetRegistry tool via tools/call", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> ToolArguments = MakeShared<FJsonObject>();
				ToolArguments->SetStringField(TEXT("Name"), TEXT("World"));

				TSharedRef<FJsonObject> CallParams = MakeShared<FJsonObject>();
				CallParams->SetStringField(TEXT("name"), GreetToolName);
				CallParams->SetObjectField(TEXT("arguments"), ToolArguments);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(3), CallParams);
				// tools/call returns an SSE stream; use a short activity timeout since the tool completes synchronously.
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ContentArray;
								if (TestTrue("Should have content", (*ResultObject)->TryGetArrayField(TEXT("content"), ContentArray))
									&& TestTrue("Content should have at least 1 element", ContentArray->Num() >= 1))
								{
									const TSharedPtr<FJsonObject>* ContentObject;
									if ((*ContentArray)[0]->TryGetObject(ContentObject))
									{
										FString Text;
										if ((*ContentObject)->TryGetStringField(TEXT("text"), Text))
										{
											TestTrue("Tool result should contain greeting", Text.Contains(TEXT("Hello, World!")));
										}
									}
								}
							}
						}
						else
						{
							int32 ResponseCode = Response->GetResponseCode();
							TestTrue("Should return success status for tool call", ResponseCode >= 200 && ResponseCode < 300);
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should execute ToolsetRegistry tool with numeric result via tools/call", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> ToolArguments = MakeShared<FJsonObject>();
				ToolArguments->SetNumberField(TEXT("A"), 3);
				ToolArguments->SetNumberField(TEXT("B"), 7);

				TSharedRef<FJsonObject> CallParams = MakeShared<FJsonObject>();
				CallParams->SetStringField(TEXT("name"), AddToolName);
				CallParams->SetObjectField(TEXT("arguments"), ToolArguments);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(4), CallParams);
				// tools/call returns an SSE stream; use a short activity timeout since the tool completes synchronously.
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ContentArray;
								if (TestTrue("Should have content", (*ResultObject)->TryGetArrayField(TEXT("content"), ContentArray))
									&& TestTrue("Content should have at least 1 element", ContentArray->Num() >= 1))
								{
									const TSharedPtr<FJsonObject>* ContentObject;
									if ((*ContentArray)[0]->TryGetObject(ContentObject))
									{
										FString Text;
										if ((*ContentObject)->TryGetStringField(TEXT("text"), Text))
										{
											TestTrue("Result should contain the sum", Text.Contains(TEXT("10")));
										}
									}
								}
							}
						}
						else
						{
							int32 ResponseCode = Response->GetResponseCode();
							TestTrue("Should return success status for tool call", ResponseCode >= 200 && ResponseCode < 300);
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should deduplicate when MCP and ToolsetRegistry have same tool name", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				// Register a deprecated MCP tool with the same qualified name as a ToolsetRegistry tool.
				// AddTool() rejects duplicates at registration time, so the ToolsetRegistry adapter's
				// tool (already registered via BeforeEach) takes precedence.
				TSharedPtr<FMockModelContextProtocolTool> MockMcpTool = MakeShared<FMockModelContextProtocolTool>();
				MockMcpTool->Name = GreetToolName;
				MockMcpTool->Description = TEXT("Deprecated MCP version of Greet tool");
				MockMcpTool->InputSchema = FMockModelContextProtocolTool::MakeTestInputSchema(
					{{TEXT("Name"), TEXT("string")}},
					{TEXT("Name")});

				IModelContextProtocolModule& Module = IModelContextProtocolModule::GetChecked();
				AddExpectedError(TEXT("is already registered"), EAutomationExpectedErrorFlags::Contains);
				TestFalse("Duplicate tool should be rejected by AddTool", Module.AddTool(MockMcpTool.ToSharedRef()));

				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(5));
				SendJsonRpcRequest(ListRequest, [this, Done, MockMcpTool](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ToolsArray;
								if (TestTrue("Should have tools array", (*ResultObject)->TryGetArrayField(TEXT("tools"), ToolsArray)))
								{
									int32 GreetCount = 0;
									FString GreetDescription;
									for (const TSharedPtr<FJsonValue>& ToolValue : *ToolsArray)
									{
										const TSharedPtr<FJsonObject>* ToolObject;
										if (ToolValue->TryGetObject(ToolObject))
										{
											FString ToolName;
											(*ToolObject)->TryGetStringField(TEXT("name"), ToolName);
											if (ToolName == GreetToolName)
											{
												GreetCount++;
												(*ToolObject)->TryGetStringField(TEXT("description"), GreetDescription);
											}
										}
									}
									TestEqual("Should have exactly one Greet tool (deduplicated)", GreetCount, 1);
									TestFalse("ToolsetRegistry tool should take precedence (not the deprecated MCP version)",
										GreetDescription == TEXT("Deprecated MCP version of Greet tool"));
								}
							}
						}
					}

					CleanupSession(Done);
				}, SessionId);
			}));
		});
	});

	// ---- Tool search enabled: only meta-tools are registered; toolset tools are dispatched via call_tool. ----
	Describe("With tool search enabled", [this]()
	{
		BeforeEach([this]()
		{
			SessionId.Reset();
			ConfigureMode(/*bEnableToolSearch=*/true);
		});

		AfterEach([this]()
		{
			RestoreMode();
		});

		LatentIt("should expose only meta-tools in tools/list (no toolset tools)", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(10));
				SendJsonRpcRequest(ListRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ToolsArray;
								if (TestTrue("Should have tools array", (*ResultObject)->TryGetArrayField(TEXT("tools"), ToolsArray)))
								{
									bool bFoundListToolsets = false;
									bool bFoundDescribeToolset = false;
									bool bFoundCallTool = false;
									bool bFoundGreet = false;
									bool bFoundAdd = false;
									for (const TSharedPtr<FJsonValue>& ToolValue : *ToolsArray)
									{
										const TSharedPtr<FJsonObject>* ToolObject;
										if (ToolValue->TryGetObject(ToolObject))
										{
											FString ToolName;
											(*ToolObject)->TryGetStringField(TEXT("name"), ToolName);
											if (ToolName == ListToolsetsToolName) bFoundListToolsets = true;
											else if (ToolName == DescribeToolsetToolName) bFoundDescribeToolset = true;
											else if (ToolName == CallToolToolName) bFoundCallTool = true;
											else if (ToolName == GreetToolName) bFoundGreet = true;
											else if (ToolName == AddToolName) bFoundAdd = true;
										}
									}
									TestTrue("Should find list_toolsets tool", bFoundListToolsets);
									TestTrue("Should find describe_toolset tool", bFoundDescribeToolset);
									TestTrue("Should find call_tool tool", bFoundCallTool);
									TestFalse("Should NOT find Greet tool (toolset tools are not registered)", bFoundGreet);
									TestFalse("Should NOT find Add tool (toolset tools are not registered)", bFoundAdd);
								}
							}
						}
					}
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should list toolsets via list_toolsets", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> CallParams = MakeShared<FJsonObject>();
				CallParams->SetStringField(TEXT("name"), ListToolsetsToolName);
				CallParams->SetObjectField(TEXT("arguments"), MakeShared<FJsonObject>());

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(12), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ContentArray;
								if (TestTrue("Should have content", (*ResultObject)->TryGetArrayField(TEXT("content"), ContentArray))
									&& TestTrue("Content should have at least 1 element", ContentArray->Num() >= 1))
								{
									const TSharedPtr<FJsonObject>* ContentObject;
									if ((*ContentArray)[0]->TryGetObject(ContentObject))
									{
										FString Text;
										if ((*ContentObject)->TryGetStringField(TEXT("text"), Text))
										{
											TestTrue("Should list mock toolset name", Text.Contains(TEXT("MockToolsetDefinition")));
										}
									}
								}
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should describe toolset via describe_toolset", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> ToolArguments = MakeShared<FJsonObject>();
				ToolArguments->SetStringField(TEXT("toolset_name"), MockToolsetName);

				TSharedRef<FJsonObject> CallParams = MakeShared<FJsonObject>();
				CallParams->SetStringField(TEXT("name"), DescribeToolsetToolName);
				CallParams->SetObjectField(TEXT("arguments"), ToolArguments);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(13), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ContentArray;
								if (TestTrue("Should have content", (*ResultObject)->TryGetArrayField(TEXT("content"), ContentArray))
									&& TestTrue("Content should have at least 1 element", ContentArray->Num() >= 1))
								{
									const TSharedPtr<FJsonObject>* ContentObject;
									if ((*ContentArray)[0]->TryGetObject(ContentObject))
									{
										FString Text;
										if ((*ContentObject)->TryGetStringField(TEXT("text"), Text))
										{
											TestTrue("Should contain Greet tool info", Text.Contains(TEXT("Greet")));
											TestTrue("Should contain Add tool info", Text.Contains(TEXT("Add")));
										}
									}
								}
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should dispatch a toolset tool through call_tool", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> ToolArguments = MakeShared<FJsonObject>();
				ToolArguments->SetStringField(TEXT("Name"), TEXT("World"));

				TSharedRef<FJsonObject> CallParams = MakeExecuteToolCallParams(MockToolsetName, GreetBareName, ToolArguments);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(20), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ContentArray;
								if (TestTrue("Should have content", (*ResultObject)->TryGetArrayField(TEXT("content"), ContentArray))
									&& TestTrue("Content should have at least 1 element", ContentArray->Num() >= 1))
								{
									const TSharedPtr<FJsonObject>* ContentObject;
									if ((*ContentArray)[0]->TryGetObject(ContentObject))
									{
										FString Text;
										if ((*ContentObject)->TryGetStringField(TEXT("text"), Text))
										{
											TestTrue("Tool result should contain greeting", Text.Contains(TEXT("Hello, World!")));
										}
									}
								}
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should dispatch a top-level tool through call_tool (no toolset_name)", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				// Omit toolset_name: call_tool should resolve list_toolsets via Module->FindTool.
				TSharedRef<FJsonObject> CallParams = MakeExecuteToolCallParams(FString(), ListToolsetsToolName, /*ToolArguments=*/nullptr);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(21), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ContentArray;
								if (TestTrue("Should have content", (*ResultObject)->TryGetArrayField(TEXT("content"), ContentArray))
									&& TestTrue("Content should have at least 1 element", ContentArray->Num() >= 1))
								{
									const TSharedPtr<FJsonObject>* ContentObject;
									if ((*ContentArray)[0]->TryGetObject(ContentObject))
									{
										FString Text;
										(*ContentObject)->TryGetStringField(TEXT("text"), Text);
										TestTrue("Should list mock toolset name", Text.Contains(TEXT("MockToolsetDefinition")));
									}
								}
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should return an error when toolset_name is unknown", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				// FToolsetRegistry::ExecuteTool logs at Error level for unknown toolset; mark that line as expected so it does not fail the test.
				AddExpectedError(TEXT("Toolset 'NonexistentToolset' not found"), EAutomationExpectedErrorFlags::Contains);

				TSharedRef<FJsonObject> CallParams = MakeExecuteToolCallParams(TEXT("NonexistentToolset"), GreetBareName, /*ToolArguments=*/nullptr);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(22), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								bool bIsError = false;
								(*ResultObject)->TryGetBoolField(TEXT("isError"), bIsError);
								TestTrue("Should return error for unknown toolset", bIsError);
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should reject self-dispatch (top-level call_tool calling itself)", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> CallParams = MakeExecuteToolCallParams(FString(), CallToolToolName, /*ToolArguments=*/nullptr);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(26), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								bool bIsError = false;
								(*ResultObject)->TryGetBoolField(TEXT("isError"), bIsError);
								TestTrue("Should return error for top-level self-dispatch", bIsError);
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should reject self-dispatch even when a toolset_name is supplied", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				// Even if the LLM tries to disguise self-dispatch with a (meaningless) toolset_name, the guard must still reject it.
				TSharedRef<FJsonObject> CallParams = MakeExecuteToolCallParams(MockToolsetName, CallToolToolName, /*ToolArguments=*/nullptr);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(27), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								bool bIsError = false;
								(*ResultObject)->TryGetBoolField(TEXT("isError"), bIsError);
								TestTrue("Should return error for toolset-qualified self-dispatch", bIsError);
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should return an error when tool_name is unknown within a known toolset", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> CallParams = MakeExecuteToolCallParams(MockToolsetName, TEXT("NonexistentTool"), /*ToolArguments=*/nullptr);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(23), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								bool bIsError = false;
								(*ResultObject)->TryGetBoolField(TEXT("isError"), bIsError);
								TestTrue("Should return error for unknown tool within known toolset", bIsError);
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should return an error when tool_name is missing", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> ExecuteArgs = MakeShared<FJsonObject>();
				ExecuteArgs->SetStringField(TEXT("toolset_name"), MockToolsetName);
				// Deliberately omit tool_name.

				TSharedRef<FJsonObject> CallParams = MakeShared<FJsonObject>();
				CallParams->SetStringField(TEXT("name"), CallToolToolName);
				CallParams->SetObjectField(TEXT("arguments"), ExecuteArgs);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(24), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								bool bIsError = false;
								(*ResultObject)->TryGetBoolField(TEXT("isError"), bIsError);
								TestTrue("Should return error when tool_name is missing", bIsError);
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});

		LatentIt("should default arguments to {} when omitted", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				// list_toolsets accepts an empty input schema; calling it through call_tool with no arguments must succeed.
				TSharedRef<FJsonObject> ExecuteArgs = MakeShared<FJsonObject>();
				ExecuteArgs->SetStringField(TEXT("tool_name"), ListToolsetsToolName);
				// Deliberately omit arguments.

				TSharedRef<FJsonObject> CallParams = MakeShared<FJsonObject>();
				CallParams->SetStringField(TEXT("name"), CallToolToolName);
				CallParams->SetObjectField(TEXT("arguments"), ExecuteArgs);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(25), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (JsonResponse.IsValid())
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								bool bIsError = false;
								(*ResultObject)->TryGetBoolField(TEXT("isError"), bIsError);
								TestFalse("call_tool with omitted arguments should not error", bIsError);
							}
						}
					}
					CleanupSession(Done);
				}, SessionId, 2.0f);
			}));
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS && WITH_EDITOR
