// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelContextProtocolEngine : ModuleRules
{
	public ModelContextProtocolEngine(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"DeveloperSettings",
				"ModelContextProtocol"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Analytics",
				"AnalyticsET",
				"Engine",
				"HTTPServer",
				"Json",
				"JsonUtilities"
			});

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"BlueprintGraph",
					"JsonUtilitiesEditor",
					"UnrealEd"
				});
		}
	}
}
