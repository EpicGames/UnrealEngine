// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsET.h"
#include "AnalyticsEventAttribute.h"
#include "EngineAnalytics.h"
#include "IAnalyticsProviderET.h"

namespace UE::ModelContextProtocol::Private
{
	/**
	 * IAnalyticsProviderET implementation dedicated to forwarding RecordEvent calls to FEngineAnalytics.
	 *
	 * FEngineAnalytics owns a real IAnalyticsProviderET internally but exposes it only by reference
	 * (FEngineAnalytics::GetProvider()). Wrapping that reference in a TSharedPtr with a no-op deleter
	 * produces a dangling reference if the underlying provider is destroyed first. This proxy owns no
	 * analytics state of its own (just the empty-default fields used when callers unexpectedly query
	 * the unused parts of the interface); callers own it via a normal TSharedPtr with clean lifetime
	 * semantics.
	 *
	 * RecordEvent is the only load-bearing method. The remaining IAnalyticsProviderET / IAnalyticsProvider
	 * surface is stubbed to safe defaults rather than forwarded: MCP never calls them, and forwarding
	 * would expose a silent path to mutate engine-global analytics state through a proxy that callers
	 * might mistake for a pass-through. This proxy is not intended to be a general-purpose
	 * FEngineAnalytics facade.
	 */
	class FEngineAnalyticsProviderProxy : public IAnalyticsProviderET
	{
	public:
		//~ IAnalyticsProviderET
		virtual void RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override
		{
			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(MoveTemp(EventName), Attributes);
			}
		}

		virtual void RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, EAnalyticsRecordEventMode Mode) override
		{
			if (FEngineAnalytics::IsAvailable())
			{
				FEngineAnalytics::GetProvider().RecordEvent(MoveTemp(EventName), Attributes, Mode);
			}
		}

		// Unused IAnalyticsProviderET members: safe defaults, no forwarding.
		virtual void SetAppID(FString&&) override {}
		virtual void SetAppVersion(FString&&) override {}
		virtual bool StartSession(FString, const TArray<FAnalyticsEventAttribute>&) override { return false; }
		virtual bool ShouldRecordEvent(const FString&) const override { return true; }
		virtual void SetUrlDomain(const FString&, const TArray<FString>&) override {}
		virtual void SetUrlPath(const FString&) override {}
		virtual void SetHeader(const FString&, const FString&) override {}
		virtual void SetEventCallback(const OnEventRecorded&) override {}
		virtual void BlockUntilFlushed(float) override {}
		virtual const FAnalyticsET::Config& GetConfig() const override { return EmptyConfig; }
		virtual void SetShouldRecordEventFunc(const ShouldRecordEventFunction&) override {}
		virtual FOnPreAnalyticsEventProcessed& OnPreAnalyticsEventProcessed() override { return PreProcessedDelegate; }
		virtual FOnAnalyticsEventQueued& OnAnalyticsEventQueued() override { return QueuedDelegate; }

		// Unused IAnalyticsProvider members: safe defaults, no forwarding.
		virtual bool StartSession(const TArray<FAnalyticsEventAttribute>&) override { return false; }
		virtual void EndSession() override {}
		virtual FString GetSessionID() const override { return FString(); }
		virtual bool SetSessionID(const FString&) override { return false; }
		virtual void FlushEvents() override {}
		virtual void SetUserID(const FString&) override {}
		virtual FString GetUserID() const override { return FString(); }
		virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&&) override {}
		virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override { return {}; }
		virtual int32 GetDefaultEventAttributeCount() const override { return 0; }
		virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int) const override { return FAnalyticsEventAttribute(FString(), FString()); }

	private:
		FAnalyticsET::Config EmptyConfig;
		FOnPreAnalyticsEventProcessed PreProcessedDelegate;
		FOnAnalyticsEventQueued QueuedDelegate;
	};
}
