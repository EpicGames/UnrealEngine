// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolClientConfig.h"
#include "ModelContextProtocol.h"

#include "Dom/JsonObject.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

namespace UE::ModelContextProtocol::Private
{
	struct FClientConfigDescriptor
	{
		FString RelativeFilePath;
		FString ServersRootKey;
		FString UrlFieldName;
		bool bIncludeTypeField = false;
		bool bIsToml = false;
	};

	FClientConfigDescriptor GetClientConfigDescriptor(EModelContextProtocolClient Client)
	{
		switch (Client)
		{
		case EModelContextProtocolClient::ClaudeCode:
			return { TEXT(".mcp.json"), TEXT("mcpServers"), TEXT("url"), /*bIncludeTypeField*/ true };
		case EModelContextProtocolClient::Cursor:
			return { TEXT(".cursor/mcp.json"), TEXT("mcpServers"), TEXT("url") };
		case EModelContextProtocolClient::VSCode:
			return { TEXT(".vscode/mcp.json"), TEXT("servers"), TEXT("url"), /*bIncludeTypeField*/ true };
		case EModelContextProtocolClient::Gemini:
			return { TEXT(".gemini/settings.json"), TEXT("mcpServers"), TEXT("httpUrl") };
		case EModelContextProtocolClient::Codex:
			return { TEXT(".codex/config.toml"), TEXT("mcp_servers"), TEXT("url"), /*bIncludeTypeField*/ false, /*bIsToml*/ true };
		default:
			return {};
		}
	}

	static constexpr const TCHAR* ServerEntryName = UE::ModelContextProtocol::DefaultServerName;

	bool WriteJsonClientConfiguration(const FString& FilePath, const FClientConfigDescriptor& Descriptor, const FString& ServerUrl)
	{
		// Read existing file if present
		TSharedPtr<FJsonObject> RootObject;
		FString ExistingContent;
		if (FFileHelper::LoadFileToString(ExistingContent, *FilePath))
		{
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ExistingContent);
			if (!FJsonSerializer::Deserialize(Reader, RootObject) || !RootObject.IsValid())
			{
				UE_LOGF(LogModelContextProtocol, Warning, "Existing file contains malformed JSON, overwriting: %ls", *FilePath);
				RootObject.Reset();
			}
		}

		if (!RootObject.IsValid())
		{
			RootObject = MakeShared<FJsonObject>();
		}

		// Get or create the servers object under the root key
		const TSharedPtr<FJsonObject>* ExistingServersObject = nullptr;
		TSharedPtr<FJsonObject> ServersObject;
		if (RootObject->TryGetObjectField(Descriptor.ServersRootKey, ExistingServersObject) && ExistingServersObject)
		{
			ServersObject = *ExistingServersObject;
		}
		if (!ServersObject.IsValid())
		{
			ServersObject = MakeShared<FJsonObject>();
		}

		// Create the server entry
		TSharedPtr<FJsonObject> ServerEntry = MakeShared<FJsonObject>();
		if (Descriptor.bIncludeTypeField)
		{
			ServerEntry->SetStringField(TEXT("type"), TEXT("http"));
		}
		ServerEntry->SetStringField(Descriptor.UrlFieldName, ServerUrl);

		ServersObject->SetObjectField(ServerEntryName, ServerEntry);
		RootObject->SetObjectField(Descriptor.ServersRootKey, ServersObject);

		// Write with pretty printing
		FString OutputString;
		TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> Writer =
			TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&OutputString);
		if (!FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer) || OutputString.IsEmpty())
		{
			UE_LOGF(LogModelContextProtocol, Warning, "Failed to serialize JSON for: %ls", *FilePath);
			return false;
		}

		// Ensure parent directory exists
		const FString Directory = FPaths::GetPath(FilePath);
		if (!FPlatformFileManager::Get().GetPlatformFile().CreateDirectoryTree(*Directory))
		{
			UE_LOGF(LogModelContextProtocol, Warning, "Failed to create directory: %ls", *Directory);
			return false;
		}

		if (FFileHelper::SaveStringToFile(OutputString, *FilePath))
		{
			UE_LOGF(LogModelContextProtocol, Display, "MCP client configuration written to: %ls", *FilePath);
			return true;
		}

		UE_LOGF(LogModelContextProtocol, Warning, "Failed to write MCP client configuration to: %ls", *FilePath);
		return false;
	}

	bool WriteTomlClientConfiguration(const FString& FilePath, const FString& ServerUrl)
	{
		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*FilePath))
		{
			UE_LOGF(LogModelContextProtocol, Error, "Codex configuration already exists at: %ls. TOML format does not support safe upsert — edit manually to add [mcp_servers.%ls].", *FilePath, ServerEntryName);
			return false;
		}

		// Ensure parent directory exists
		const FString Directory = FPaths::GetPath(FilePath);
		if (!FPlatformFileManager::Get().GetPlatformFile().CreateDirectoryTree(*Directory))
		{
			UE_LOGF(LogModelContextProtocol, Warning, "Failed to create directory: %ls", *Directory);
			return false;
		}

		FString TomlContent;
		TomlContent += FString::Printf(TEXT("[mcp_servers.%s]\n"), ServerEntryName);
		TomlContent += FString::Printf(TEXT("url = \"%s\"\n"), *ServerUrl);

		if (FFileHelper::SaveStringToFile(TomlContent, *FilePath))
		{
			UE_LOGF(LogModelContextProtocol, Display, "MCP client configuration written to: %ls", *FilePath);
			return true;
		}

		UE_LOGF(LogModelContextProtocol, Warning, "Failed to write MCP client configuration to: %ls", *FilePath);
		return false;
	}
}

bool UE::ModelContextProtocol::WriteClientConfiguration(EModelContextProtocolClient Client, uint32 Port, const FString& UrlPath, const FString& BaseDirectory)
{
	const Private::FClientConfigDescriptor Descriptor = Private::GetClientConfigDescriptor(Client);
	if (Descriptor.RelativeFilePath.IsEmpty())
	{
		UE_LOGF(LogModelContextProtocol, Warning, "Unknown MCP client type");
		return false;
	}

	const FString ServerUrl = FString::Printf(TEXT("http://127.0.0.1:%u%s"), Port, *UrlPath);
	// In source builds, FPaths::RootDir() returns the workspace root (the directory containing Engine/),
	// which is where AI coding clients expect config files.
	// In installed/launcher builds, RootDir() points to the engine installation, so fall back to ProjectDir().
	const FString RootDirectory = BaseDirectory.IsEmpty()
		? (FApp::IsEngineInstalled() ? FPaths::ProjectDir() : FPaths::RootDir())
		: BaseDirectory;
	const FString FilePath = FPaths::Combine(RootDirectory, Descriptor.RelativeFilePath);

	if (Descriptor.bIsToml)
	{
		return Private::WriteTomlClientConfiguration(FilePath, ServerUrl);
	}
	else
	{
		return Private::WriteJsonClientConfiguration(FilePath, Descriptor, ServerUrl);
	}
}

int32 UE::ModelContextProtocol::WriteAllClientConfigurations(uint32 Port, const FString& UrlPath, const FString& BaseDirectory)
{
	int32 SuccessCount = 0;

	for (uint8 Index = 0; Index <= static_cast<uint8>(EModelContextProtocolClient::Codex); ++Index)
	{
		if (WriteClientConfiguration(static_cast<EModelContextProtocolClient>(Index), Port, UrlPath, BaseDirectory))
		{
			++SuccessCount;
		}
	}

	return SuccessCount;
}
