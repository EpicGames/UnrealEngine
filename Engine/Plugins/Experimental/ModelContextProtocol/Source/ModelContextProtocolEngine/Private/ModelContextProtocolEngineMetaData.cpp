// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolEngineMetaData.h"

#include "JsonSchema/JsonSchemaEditorMetadata.h"
#include "JsonSchema/JsonSchemaPropertyFilter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#if WITH_EDITORONLY_DATA
#include "EdGraphSchema_K2.h"
#include "JsonSchema/JsonSchemaGeneratorEditor.h"
#endif // WITH_EDITORONLY_DATA

namespace UE::ModelContextProtocol
{

#if WITH_EDITORONLY_DATA
FModelContextProtocolFunctionMetaData CollectFunctionMetaData(const UFunction* Function, const FJsonSchemaPropertyFilter& PropertyFilter)
{
	FModelContextProtocolFunctionMetaData Result;

	if (!Function)
	{
		return Result;
	}

	FJsonSchemaEditorMetadata EditorMetadata = FJsonSchemaGeneratorEditor::UStructToJsonSchemaMetadata(Function, PropertyFilter, nullptr);

	if (EditorMetadata.RootStructMetadata.IsValid())
	{
		FString DescriptionString;
		if (EditorMetadata.RootStructMetadata->TryGetStringField(TEXT("description"), DescriptionString) && !DescriptionString.IsEmpty())
		{
			Result.Description = FText::FromString(DescriptionString);
		}
	}

	if (Function->HasMetaData(FBlueprintMetadata::MD_WorldContext))
	{
		Result.WorldContext = Function->GetMetaData(FBlueprintMetadata::MD_WorldContext);
	}

	for (const auto& [PropertyPath, PropertyMetadataJsonObject] : EditorMetadata.GetPropertyMemberPathToPropertyMetadataMap())
	{
		if (!PropertyMetadataJsonObject.IsValid())
		{
			continue;
		}

		FModelContextProtocolPropertyMetaData PropertyMetaData;

		FString DescriptionString;
		if (PropertyMetadataJsonObject->TryGetStringField(TEXT("description"), DescriptionString) && !DescriptionString.IsEmpty())
		{
			PropertyMetaData.Description = FText::FromString(DescriptionString);
		}

		if (PropertyMetadataJsonObject->HasField(TEXT("default")))
		{
			const TSharedPtr<FJsonValue>& DefaultValue = PropertyMetadataJsonObject->Values.FindChecked(TEXT("default"));
			if (DefaultValue.IsValid())
			{
				PropertyMetaData.DefaultValueJsonType = static_cast<uint8>(DefaultValue->Type);
				switch (DefaultValue->Type)
				{
				case EJson::String:
					PropertyMetaData.DefaultValue = DefaultValue->AsString();
					break;
				case EJson::Number:
					PropertyMetaData.DefaultValue = FString::SanitizeFloat(DefaultValue->AsNumber());
					break;
				case EJson::Boolean:
					PropertyMetaData.DefaultValue = DefaultValue->AsBool() ? TEXT("true") : TEXT("false");
					break;
				case EJson::Null:
					PropertyMetaData.DefaultValue = TEXT("None");
					break;
				case EJson::Object:
				case EJson::Array:
					{
						FString JsonString;
						TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
						FJsonSerializer::Serialize(DefaultValue, TEXT(""), Writer);
						PropertyMetaData.DefaultValue = JsonString;
					}
					break;
				default:
					break;
				}
			}
		}

		double MinValue;
		if (PropertyMetadataJsonObject->TryGetNumberField(TEXT("minimum"), MinValue))
		{
			PropertyMetaData.ClampMin = MinValue;
		}

		double MaxValue;
		if (PropertyMetadataJsonObject->TryGetNumberField(TEXT("maximum"), MaxValue))
		{
			PropertyMetaData.ClampMax = MaxValue;
		}

		Result.PropertyMetaData.Add(PropertyPath, MoveTemp(PropertyMetaData));
	}

	return Result;
}
#endif // WITH_EDITORONLY_DATA

FJsonSchemaEditorMetadata ConvertToCachedEditorMetadata(const FModelContextProtocolFunctionMetaData& FunctionMetaData)
{
	FJsonSchemaEditorMetadata EditorMetadata;

	// Set root struct description
	if (FunctionMetaData.Description.IsSet())
	{
		EditorMetadata.RootStructMetadata = MakeShared<FJsonObject>();
		EditorMetadata.RootStructMetadata->SetStringField(TEXT("description"), FunctionMetaData.Description->ToString());
	}

	// Convert per-property metadata
	FJsonSchemaEditorMetadata::FJsonSchemaPropertyMemberPathToPropertyMetadataMap& PropertyMap = EditorMetadata.GetPropertyMemberPathToPropertyMetadataMap();
	for (const auto& [PropertyPath, PropertyMetaData] : FunctionMetaData.PropertyMetaData)
	{
		TSharedPtr<FJsonObject> PropertyJsonObject = MakeShared<FJsonObject>();

		if (PropertyMetaData.Description.IsSet())
		{
			PropertyJsonObject->SetStringField(TEXT("description"), PropertyMetaData.Description->ToString());
		}

		if (PropertyMetaData.DefaultValue.IsSet())
		{
			const FString& DefaultString = *PropertyMetaData.DefaultValue;
			const EJson OriginalType = PropertyMetaData.DefaultValueJsonType.IsSet()
				? static_cast<EJson>(*PropertyMetaData.DefaultValueJsonType)
				: EJson::String;

			switch (OriginalType)
			{
			case EJson::Boolean:
				PropertyJsonObject->SetBoolField(TEXT("default"), DefaultString == TEXT("true"));
				break;
			case EJson::Null:
				PropertyJsonObject->SetField(TEXT("default"), MakeShared<FJsonValueNull>());
				break;
			case EJson::Number:
				{
					double NumericValue;
					if (LexTryParseString(NumericValue, *DefaultString))
					{
						PropertyJsonObject->SetNumberField(TEXT("default"), NumericValue);
					}
					else
					{
						PropertyJsonObject->SetStringField(TEXT("default"), DefaultString);
					}
				}
				break;
			case EJson::Object:
			case EJson::Array:
				{
					TSharedPtr<FJsonValue> ParsedValue;
					TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(DefaultString);
					if (FJsonSerializer::Deserialize(Reader, ParsedValue) && ParsedValue.IsValid())
					{
						PropertyJsonObject->SetField(TEXT("default"), ParsedValue);
					}
					else
					{
						PropertyJsonObject->SetStringField(TEXT("default"), DefaultString);
					}
				}
				break;
			default:
				PropertyJsonObject->SetStringField(TEXT("default"), DefaultString);
				break;
			}
		}

		if (PropertyMetaData.ClampMin.IsSet())
		{
			PropertyJsonObject->SetNumberField(TEXT("minimum"), *PropertyMetaData.ClampMin);
		}

		if (PropertyMetaData.ClampMax.IsSet())
		{
			PropertyJsonObject->SetNumberField(TEXT("maximum"), *PropertyMetaData.ClampMax);
		}

		PropertyMap.Add(PropertyPath, PropertyJsonObject);
	}

	return EditorMetadata;
}

}
