// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineAnalyticsProviderProxy.h"
#include "IModelContextProtocolModule.h"
#include "ModelContextProtocol.h"
#include "ModelContextProtocolClientConfig.h"
#include "ModelContextProtocolSettings.h"

#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

namespace UE::ModelContextProtocol::Private
{
	TOptional<EModelContextProtocolClient> ParseClientName(const FString& Name)
	{
		if (Name.Equals(TEXT("ClaudeCode"), ESearchCase::IgnoreCase) || Name.Equals(TEXT("Claude"), ESearchCase::IgnoreCase))
		{
			return EModelContextProtocolClient::ClaudeCode;
		}
		if (Name.Equals(TEXT("Cursor"), ESearchCase::IgnoreCase))
		{
			return EModelContextProtocolClient::Cursor;
		}
		if (Name.Equals(TEXT("VSCode"), ESearchCase::IgnoreCase) || Name.Equals(TEXT("Copilot"), ESearchCase::IgnoreCase))
		{
			return EModelContextProtocolClient::VSCode;
		}
		if (Name.Equals(TEXT("Gemini"), ESearchCase::IgnoreCase))
		{
			return EModelContextProtocolClient::Gemini;
		}
		if (Name.Equals(TEXT("Codex"), ESearchCase::IgnoreCase))
		{
			return EModelContextProtocolClient::Codex;
		}
		return {};
	}

	FAutoConsoleCommand CommandGenerateClientConfig = FAutoConsoleCommand(
		TEXT("ModelContextProtocol.GenerateClientConfig"),
		TEXT("Generates MCP client configuration file(s). Usage: ModelContextProtocol.GenerateClientConfig <ClaudeCode|Cursor|VSCode|Gemini|Codex|All>"),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (Args.IsEmpty())
			{
				UE_LOGF(LogModelContextProtocol, Warning, "Usage: ModelContextProtocol.GenerateClientConfig <ClaudeCode|Cursor|VSCode|Gemini|Codex|All>");
				return;
			}

			const uint32 Port = GetServerPortNumber();
			const FString UrlPath = GetServerUrlPath();

			if (Args[0].Equals(TEXT("All"), ESearchCase::IgnoreCase))
			{
				const int32 Count = WriteAllClientConfigurations(Port, UrlPath);
				UE_LOGF(LogModelContextProtocol, Display, "Generated %d MCP client configuration file(s)", Count);
			}
			else
			{
				TOptional<EModelContextProtocolClient> Client = ParseClientName(Args[0]);
				if (!Client.IsSet())
				{
					UE_LOGF(LogModelContextProtocol, Warning, "Unknown client \"%ls\". Supported: ClaudeCode, Cursor, VSCode, Gemini, Codex, All", *Args[0]);
					return;
				}
				WriteClientConfiguration(Client.GetValue(), Port, UrlPath);
			}
		}),
		ECVF_Cheat);
}

class FModelContextProtocolEngineModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		// FEngineAnalytics is initialized during post-engine-init, so defer the wiring.
		// Consumers can override the provider at any time via IModelContextProtocolModule::SetAnalyticsProvider.
		PostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
		{
			RegisterDefaultAnalyticsProvider();
		});
	}

	virtual void ShutdownModule() override
	{
		if (PostEngineInitHandle.IsValid())
		{
			FCoreDelegates::GetOnPostEngineInit().Remove(PostEngineInitHandle);
			PostEngineInitHandle.Reset();
		}

		// Release the proxy we installed. If a consumer overrode the provider, we leave theirs in place.
		if (EngineAnalyticsProviderProxy.IsValid())
		{
			if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
			{
				if (Module->GetAnalyticsProvider() == EngineAnalyticsProviderProxy)
				{
					Module->SetAnalyticsProvider(nullptr);
				}
			}
			EngineAnalyticsProviderProxy.Reset();
		}
	}

private:
	void RegisterDefaultAnalyticsProvider()
	{
		IModelContextProtocolModule* Module = IModelContextProtocolModule::Get();
		if (!Module)
		{
			return;
		}

		if (Module->GetAnalyticsProvider().IsValid())
		{
			UE_LOGF(LogModelContextProtocol, Log, "Analytics provider already set; not overriding with FEngineAnalytics proxy default.");
			return;
		}

		// The proxy forwards recording to FEngineAnalytics while guarding against its availability.
		// We own the proxy outright -- no ownership tricks, no dangling-reference risk.
		EngineAnalyticsProviderProxy = MakeShared<UE::ModelContextProtocol::Private::FEngineAnalyticsProviderProxy>();
		Module->SetAnalyticsProvider(EngineAnalyticsProviderProxy);

		UE_LOGF(LogModelContextProtocol, Log, "Registered FEngineAnalytics proxy as the default MCP analytics provider.");
	}

	FDelegateHandle PostEngineInitHandle;
	TSharedPtr<UE::ModelContextProtocol::Private::FEngineAnalyticsProviderProxy> EngineAnalyticsProviderProxy;
};

IMPLEMENT_MODULE(FModelContextProtocolEngineModule, ModelContextProtocolEngine);
