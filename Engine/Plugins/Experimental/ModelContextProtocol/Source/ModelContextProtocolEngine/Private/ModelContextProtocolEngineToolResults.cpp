// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolEngineToolResults.h"
#include "ModelContextProtocol.h"

#include "Engine/Texture2D.h"
#include "ImageUtils.h"
#include "Misc/ScopeExit.h"
#include "TextureResource.h"
#include "Sound/SoundWave.h"

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeImageResult(const UTexture2D* Texture, const TCHAR* ToFormatExtension, EModelContextProtocolAudience Audience)
{
	if (!Texture)
	{
		return FModelContextProtocolToolResult();
	}

	if (const FTexturePlatformData* TexturePlatformData = Texture->GetPlatformData();
		ensure(TexturePlatformData) && ensure(!TexturePlatformData->Mips.IsEmpty()) && ensure(TexturePlatformData->PixelFormat == PF_B8G8R8A8))
	{
		const FTexture2DMipMap& Mip0 = TexturePlatformData->Mips[0];
		if (Mip0.SizeX > 0 && Mip0.SizeY > 0)
		{
			const void* Mip0Data = Mip0.BulkData.LockReadOnly();
			if (Mip0Data)
			{
				ON_SCOPE_EXIT{ Mip0.BulkData.Unlock(); };
				return MakeImageResult(FImageView(static_cast<const FColor*>(Mip0Data), Mip0.SizeX, Mip0.SizeY), ToFormatExtension, Audience);
			}
		}
	}

	return FModelContextProtocolToolResult();
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeImageResult(const FImageView& Image, const TCHAR* ToFormatExtension, EModelContextProtocolAudience Audience)
{
	if (!ensure(Image.IsImageInfoValid()))
	{
		return FModelContextProtocolToolResult();
	}

	TArray64<uint8> CompressedData;
	FImageUtils::CompressImage(CompressedData, ToFormatExtension, Image);

	FStringView Format(ToFormatExtension);
	if (Format.StartsWith(TEXT('.')))
	{
		Format.RightChopInline(1);
	}

	return MakeImageResult(FString(TEXT("image/")) + Format, CompressedData, Audience);
}

FModelContextProtocolToolResult UE::ModelContextProtocol::MakeAudioResult(USoundWave* SoundWave)
{
	if (!SoundWave)
	{
		return FModelContextProtocolToolResult();
	}

#if WITH_EDITOR
	if (UE::ModelContextProtocol::bAudioResultOggFormat)
	{
		if (FByteBulkData* BulkData = SoundWave->GetCompressedData("OGG"))
		{
			TArray<uint8> RawData;
			RawData.Append(BulkData->GetCopyAsBuffer(BulkData->GetElementCount(), true).GetView());

			return MakeAudioResult(TEXT("audio/ogg"), RawData);
		}
	}
	else
	{
		TFuture<FSharedBuffer> FutureBuffer = SoundWave->RawData.GetPayload(USoundWave::FEditorAudioBulkData::EPayloadFlags::ReturnCopyWithTransformationData);
		FutureBuffer.Wait();
		if (FutureBuffer.IsReady())
		{
			FSharedBuffer Buffer = FutureBuffer.Get();
			if (Buffer.GetSize() > 0)
			{
				return MakeAudioResult(TEXT("audio/wav"), MakeArrayView<uint8>(static_cast<uint8*>(const_cast<void*>(Buffer.GetData())), Buffer.GetSize()));
			}
		}
	}
#endif // WITH_EDITOR

	return FModelContextProtocolToolResult();
}
