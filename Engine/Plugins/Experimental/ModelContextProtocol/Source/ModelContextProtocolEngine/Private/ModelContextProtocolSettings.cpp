// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolSettings.h"

#include "HttpPath.h"
#include "Misc/CommandLine.h"
#include "ModelContextProtocol.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelContextProtocolSettings)

FString UE::ModelContextProtocol::GetServerUrlPath()
{
	return GetDefault<UModelContextProtocolSettings>()->ServerUrlPath;
}

uint32 UE::ModelContextProtocol::GetServerPortNumber()
{
	uint32 Port = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("ModelContextProtocolPort="), Port))
	{
		if (Port >= 1 && Port <= 65535)
		{
			return Port;
		}
		UE_LOGF(LogModelContextProtocol, Warning, "Invalid command-line port %u. Must be 1-65535. Falling back to settings.", Port);
	}

	return GetDefault<UModelContextProtocolSettings>()->ServerPortNumber;
}

bool UE::ModelContextProtocol::ShouldAutoStartServer()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("ModelContextProtocolStartServer")))
	{
		return true;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("StartModelContextProtocolServer")))
	{
		static bool bDeprecationWarned = false;
		if (!bDeprecationWarned)
		{
			bDeprecationWarned = true;
			UE_LOGF(LogModelContextProtocol, Warning, "-StartModelContextProtocolServer is deprecated; use -ModelContextProtocolStartServer instead.");
		}
		return true;
	}

	return GetDefault<UModelContextProtocolSettings>()->bAutoStartServer;
}

void UModelContextProtocolSettings::EnforceValidServerUrlPath(FString& InOutPath)
{
	if (!FHttpPath(InOutPath).IsValidPath())
	{
		UE_LOGF(LogModelContextProtocol, Warning, "ServerUrlPath '%ls' is not a valid HTTP path; falling back to '%ls'.", *InOutPath, UE::ModelContextProtocol::DefaultServerUrlPath);
		InOutPath = UE::ModelContextProtocol::DefaultServerUrlPath;
	}
}

#if WITH_EDITOR
void UModelContextProtocolSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Don't fight the user while they are still typing - only act on the final value.
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UModelContextProtocolSettings, ServerUrlPath))
	{
		EnforceValidServerUrlPath(ServerUrlPath);
	}
}
#endif
