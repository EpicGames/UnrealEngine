// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolAsyncAction.h"
#include "IModelContextProtocolModule.h"
#include "ModelContextProtocolEngineMetaData.h"
#include "ModelContextProtocolSession.h"
#include "ModelContextProtocolToolUtils.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "JsonSchema/JsonSchemaGenerator.h"
#include "JsonSchema/JsonSchemaPropertyFilter.h"

#if WITH_EDITORONLY_DATA
#include "JsonSchema/JsonSchemaGeneratorEditor.h"
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
#include "Editor/EditorEngine.h"
#endif


#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelContextProtocolToolAsyncAction)

void UModelContextProtocolToolAsyncAction::PostInitProperties()
{
	Super::PostInitProperties();

	// Register with IModelContextProtocolModule
	if (bAutoRegisterTool && HasAnyFlags(RF_ClassDefaultObject) && !GetClass()->HasAnyClassFlags(CLASS_Abstract) && GetClass()->HasAnyClassFlags(CLASS_Native))
	{
		RegisterTool();
	}
}

void UModelContextProtocolToolAsyncAction::FinishDestroy()
{
	// Remove from global tool libraries list
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DeregisterTool();
	}

	Super::FinishDestroy();
}

void UModelContextProtocolToolAsyncAction::SetReadyToDestroy()
{
	Super::SetReadyToDestroy();

	OnAsyncToolComplete.Broadcast(this);
	OnAsyncToolComplete.Clear();
}

void UModelContextProtocolToolAsyncAction::RegisterTool()
{
	DeregisterTool();

	check(!Tool.IsValid());
	check(HasAnyFlags(RF_ClassDefaultObject));

#if WITH_EDITOR
	CollectToolMetaData();
#endif

	IModelContextProtocolModule& ModelContextProtocolModule = IModelContextProtocolModule::GetChecked();

	if (UFunction* Function = GetClass()->FindFunctionByName(GetToolFunctionName(), EIncludeSuperFlag::ExcludeSuper))
	{
		Tool = MakeShared<FModelContextProtocolAsyncActionTool>(this, Function);

		if (Tool.IsValid())
		{
			ModelContextProtocolModule.AddTool(Tool.ToSharedRef());
		}
	}

	// Re-register tools on refresh
	ModelContextProtocolModule.OnRefreshTools().AddWeakLambda(this, [this]()
		{
			RegisterTool();
		});
}

void UModelContextProtocolToolAsyncAction::DeregisterTool()
{
	if (IModelContextProtocolModule* ModelContextProtocolModule = IModelContextProtocolModule::Get())
	{
		if (Tool.IsValid())
		{
			ModelContextProtocolModule->RemoveTool(Tool.ToSharedRef());
		}

		// Re-register tools on refresh
		ModelContextProtocolModule->OnRefreshTools().RemoveAll(this);
	}
	Tool.Reset();
}

#if WITH_EDITORONLY_DATA
void UModelContextProtocolToolAsyncAction::CollectToolMetaData()
{
	Modify();

	ToolFunctionMetaData = FModelContextProtocolFunctionMetaData();
	ToolResultMetaData = FModelContextProtocolFunctionMetaData();

	if (UFunction* Function = GetClass()->FindFunctionByName(GetToolFunctionName(), EIncludeSuperFlag::ExcludeSuper))
	{
		FJsonSchemaPropertyFilter FunctionFilter(/*CheckFlags*/CPF_Parm | CPF_ReturnParm | CPF_OutParm | CPF_ConstParm);
		ToolFunctionMetaData = UE::ModelContextProtocol::CollectFunctionMetaData(Function, FunctionFilter);
	}

	if (FProperty* Property = GetClass()->FindPropertyByName(GetToolResultPropertyName()))
	{
		FJsonSchemaPropertyFilter ClassFilter;
		FJsonSchemaEditorMetadata EditorMetadata = FJsonSchemaGeneratorEditor::UStructToJsonSchemaMetadata(GetClass(), ClassFilter, nullptr);

		if (EditorMetadata.RootStructMetadata.IsValid())
		{
			FString DescriptionString;
			if (EditorMetadata.RootStructMetadata->TryGetStringField(TEXT("description"), DescriptionString) && !DescriptionString.IsEmpty())
			{
				ToolResultMetaData.Description = FText::FromString(DescriptionString);
			}
		}

		for (const auto& [PropertyPath, PropertyMetadataJsonObject] : EditorMetadata.GetPropertyMemberPathToPropertyMetadataMap())
		{
			if (!PropertyMetadataJsonObject.IsValid())
			{
				continue;
			}

			FModelContextProtocolPropertyMetaData PropertyMetaData;
			FString DescriptionString;
			if (PropertyMetadataJsonObject->TryGetStringField(TEXT("description"), DescriptionString) && !DescriptionString.IsEmpty())
			{
				PropertyMetaData.Description = FText::FromString(DescriptionString);
			}

			double MinValue;
			if (PropertyMetadataJsonObject->TryGetNumberField(TEXT("minimum"), MinValue))
			{
				PropertyMetaData.ClampMin = MinValue;
			}

			double MaxValue;
			if (PropertyMetadataJsonObject->TryGetNumberField(TEXT("maximum"), MaxValue))
			{
				PropertyMetaData.ClampMax = MaxValue;
			}

			ToolResultMetaData.PropertyMetaData.Add(PropertyPath, MoveTemp(PropertyMetaData));
		}
	}
}
#endif // WITH_EDITORONLY_DATA

FModelContextProtocolAsyncActionTool::FModelContextProtocolAsyncActionTool(UModelContextProtocolToolAsyncAction* InAction, UFunction* InFunction)
	: AsyncAction(InAction)
	, Function(InFunction)
	, FunctionParamsContainer(Function)
{
	check(AsyncAction);
	check(Function);
}

FString FModelContextProtocolAsyncActionTool::GetName() const
{
	check(Function);
	return Function->GetName();
}

FString FModelContextProtocolAsyncActionTool::GetDescription() const
{
	check(AsyncAction);
	check(Function);

	const FModelContextProtocolFunctionMetaData* FunctionMetaData = AsyncAction->GetToolFunctionMetaData();
	if (FunctionMetaData && FunctionMetaData->Description.IsSet())
	{
		return FunctionMetaData->Description->ToString();
	}

	return FString();
}

TSharedPtr<FJsonObject> FModelContextProtocolAsyncActionTool::GetInputJsonSchema() const
{
	check(AsyncAction);
	check(Function);

	// Return cached?
	if (InputJsonSchema.IsValid())
	{
		return InputJsonSchema;
	}

	const FModelContextProtocolFunctionMetaData* FunctionMetaData = AsyncAction->GetToolFunctionMetaData();

	// Build property filter — skip WorldContext param as we auto-set it, and skip return/out params
	TOptional<TSet<FString>> SkipPropertyMemberPaths;
	if (FunctionMetaData && !FunctionMetaData->WorldContext.IsEmpty())
	{
		SkipPropertyMemberPaths.Emplace();
		SkipPropertyMemberPaths->Add(FunctionMetaData->WorldContext);
	}

	FJsonSchemaPropertyFilter PropertyFilter(
		/*CheckFlags*/CPF_Parm | CPF_ConstParm,
		/*SkipFlags*/CPF_ReturnParm | CPF_OutParm,
		/*RequiredPropertyMemberPaths*/{},
		SkipPropertyMemberPaths);

	// Convert cookable metadata to engine format for schema generation
	TOptional<FJsonSchemaEditorMetadata> CachedEditorMetadata;
	if (FunctionMetaData)
	{
		CachedEditorMetadata = UE::ModelContextProtocol::ConvertToCachedEditorMetadata(*FunctionMetaData);
	}

	TSharedPtr<FJsonObject> SchemaResult = FJsonSchemaGenerator::UStructToJsonSchemaObject(
		Function, PropertyFilter, CachedEditorMetadata.IsSet() ? &CachedEditorMetadata.GetValue() : nullptr);

	// UStructToJsonSchemaObject returns JSON-RPC wrapper for UFunctions: {inputSchema, outputSchema}
	// Extract inputSchema
	if (SchemaResult.IsValid())
	{
		const TSharedPtr<FJsonObject>* InputSchemaField;
		if (SchemaResult->TryGetObjectField(TEXT("inputSchema"), InputSchemaField))
		{
			InputJsonSchema = *InputSchemaField;
		}
	}

	return InputJsonSchema;
}

TSharedPtr<FJsonObject> FModelContextProtocolAsyncActionTool::GetOutputJsonSchema() const
{
	CacheResultTypeInfo();

	return OutputJsonSchema;
}

void FModelContextProtocolAsyncActionTool::RunAsync(const FModelContextProtocolToolRequestId& RequestId, const TSharedPtr<FJsonObject>& Params, const FResultCallback& OnComplete)
{
	check(AsyncAction);
	check(Function);

	CacheResultTypeInfo();

	// Prepare params
	Function->InitializeStruct(FunctionParamsContainer.GetStructMemory());
	ON_SCOPE_EXIT
	{
		Function->DestroyStruct(FunctionParamsContainer.GetStructMemory());
	};

	// Apply param defaults
	const FModelContextProtocolFunctionMetaData* FunctionMetaData = AsyncAction->GetToolFunctionMetaData();

	if (FunctionMetaData)
	{
		for (TFieldIterator<FProperty> ParamPropertyIt(Function); ParamPropertyIt; ++ParamPropertyIt)
		{
			const FProperty* ParamProperty = *ParamPropertyIt;
			FString ParamName = ParamProperty->GetAuthoredName();
			if (const FModelContextProtocolPropertyMetaData* PropertyMetaData = FunctionMetaData->PropertyMetaData.Find(*ParamName);
				PropertyMetaData && PropertyMetaData->DefaultValue.IsSet())
			{
				ParamProperty->ImportText_InContainer(**PropertyMetaData->DefaultValue, FunctionParamsContainer.GetStructMemory(), AsyncAction, PPF_None);
			}
		}
	}

	if (Params.IsValid())
	{
		// Json Params -> UFunction params container
		FText FailReason;
		if (!FJsonObjectConverter::JsonObjectToUStruct(Params.ToSharedRef(), Function, FunctionParamsContainer.GetStructMemory(), /*CheckFlags*/CPF_Parm | CPF_ConstParm, /*SkipFlags*/CPF_ReturnParm | CPF_OutParm, /*bStrictMode*/false, &FailReason))
		{
			OnComplete(UE::ModelContextProtocol::MakeErrorResult(FailReason.ToString()));
			return;
		}
	}

	// WorldContext
	if (FunctionMetaData && !FunctionMetaData->WorldContext.IsEmpty())
	{
		UObject* WorldContextObject = GWorld.GetReference();
		if (FProperty* WorldContextProperty = Function->FindPropertyByName(FName(FunctionMetaData->WorldContext)); ensure(WorldContextProperty))
		{
			if (FObjectPropertyBase* WorldContextObjectProperty = CastField<FObjectPropertyBase>(WorldContextProperty); ensure(WorldContextObjectProperty))
			{
				WorldContextObjectProperty->SetObjectPropertyValue_InContainer(FunctionParamsContainer.GetStructMemory(), WorldContextObject);
			}
		}
	}

	{
		// Execute tool
#if WITH_EDITOR
		FEditorScriptExecutionGuard ScriptExecutionGuard;
#endif
		AsyncAction->ProcessEvent(Function, FunctionParamsContainer.GetStructMemory());
	}

	if (FProperty* ReturnProperty = Function->GetReturnProperty(); ReturnProperty && ensure(Function->ReturnValueOffset != MAX_uint16))
	{
		if (FObjectProperty* ObjectReturnProperty = CastField<FObjectProperty>(ReturnProperty); ensure(ObjectReturnProperty))
		{
			if (UObject* ObjectResult = ObjectReturnProperty->GetObjectPropertyValue_InContainer(FunctionParamsContainer.GetStructMemory()))
			{
				if (UModelContextProtocolToolAsyncAction* NewAction = Cast<UModelContextProtocolToolAsyncAction>(ObjectResult))
				{
					NewAction->ToolRequestId = RequestId;

					InProgressActions.Add(NewAction);

					NewAction->OnAsyncToolComplete.AddSPLambda(this, [this, OnComplete](UModelContextProtocolToolAsyncAction* InAction)
						{
							InProgressActions.Remove(InAction);

							FModelContextProtocolToolResult Result;

							if (FProperty* ResultProperty = InAction->GetClass()->FindPropertyByName(InAction->GetToolResultPropertyName()))
							{
								check(ResultType.IsSet());

								Result = UE::ModelContextProtocol::GetToolResultFromType(*ResultType, ResultProperty, InAction, ResultProperty->GetOffset_ForInternal());
							}

							OnComplete(Result);
						});

					return;
				}
			}
		}
	}

	OnComplete(UE::ModelContextProtocol::MakeErrorResult(TEXT("Error")));
}

void FModelContextProtocolAsyncActionTool::CancelAsync(const FModelContextProtocolToolRequestId& RequestId)
{
	if (RequestId.IsValid())
	{
		for (TObjectPtr<UModelContextProtocolToolAsyncAction>& Action : InProgressActions)
		{
			if (Action && Action->ToolRequestId.IsValid() && FJsonValue::CompareEqual(*RequestId.RequestId, *Action->ToolRequestId.RequestId))
			{
				Action->Cancel();
				break;
			}
		}
	}
}

void FModelContextProtocolAsyncActionTool::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(AsyncAction);
	Collector.AddReferencedObject(Function);
	Collector.AddReferencedObjects(InProgressActions);
}

void FModelContextProtocolAsyncActionTool::CacheResultTypeInfo() const
{
	check(AsyncAction);
	check(Function);

	// Already cached?
	if (ResultType.IsSet())
	{
		return;
	}

	// Determine result type
	ResultType = EModelContextProtocolToolResultType::None;
	OutputJsonSchema.Reset();

	if (FProperty* ResultProperty = AsyncAction->GetClass()->FindPropertyByName(AsyncAction->GetToolResultPropertyName()))
	{
		ResultType = UE::ModelContextProtocol::GetToolResultType(ResultProperty, AsyncAction->GetToolResultMetaData(), OutputJsonSchema);
	}
}
