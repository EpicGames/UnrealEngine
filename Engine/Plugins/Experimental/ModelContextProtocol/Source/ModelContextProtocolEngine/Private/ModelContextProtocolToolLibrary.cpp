// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolLibrary.h"
#include "IModelContextProtocolModule.h"
#include "ModelContextProtocol.h"
#include "ModelContextProtocolEngineMetaData.h"
#include "ModelContextProtocolToolUtils.h"
#include "JsonSchema/JsonSchemaGenerator.h"
#include "JsonSchema/JsonSchemaPropertyFilter.h"
#include "Engine/World.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Editor.h"
#include "ScopedTransaction.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelContextProtocolToolLibrary)

#define LOCTEXT_NAMESPACE "ModelContextProtocolToolLibrary"

void UModelContextProtocolToolLibrary::PostInitProperties()
{
	Super::PostInitProperties();

	// Note: We only need to register tools for Native classes here, as BP classes will register later via PostLoad or PostCDOCompiled
	if (bAutoRegisterTools && HasAnyFlags(RF_ClassDefaultObject) && !GetClass()->HasAnyClassFlags(CLASS_Abstract) && GetClass()->HasAnyClassFlags(CLASS_Native))
	{
		RegisterTools();
	}
}

void UModelContextProtocolToolLibrary::PostLoad()
{
	Super::PostLoad();
	
	// Register BP classes here (native classes already registered in PostInitProperties)
	if (bAutoRegisterTools && HasAnyFlags(RF_ClassDefaultObject) && !GetClass()->HasAnyClassFlags(CLASS_Abstract) && !GetClass()->HasAnyClassFlags(CLASS_Native))
	{
		RegisterTools();
	}
}

#if WITH_EDITOR
void UModelContextProtocolToolLibrary::PostCDOCompiled(const FPostCDOCompiledContext& Context)
{
	Super::PostCDOCompiled(Context);

	// @todo Account for CDO recompilation for bAutoRegisterTools=false but where manual RegisterTools has already occurred?   
	if (bAutoRegisterTools)
	{
		RegisterTools();
	}
}
#endif // WITH_EDITOR

void UModelContextProtocolToolLibrary::FinishDestroy()
{
	// Remove from global tool libraries list
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DeregisterTools();
	}
	
	Super::FinishDestroy();
}

void UModelContextProtocolToolLibrary::RegisterTools()
{
	DeregisterTools();
	
	check(Tools.IsEmpty());
	check(HasAnyFlags(RF_ClassDefaultObject));

#if WITH_EDITOR
	CollectFunctionMetaData();
#endif

	IModelContextProtocolModule& ModelContextProtocolModule = IModelContextProtocolModule::GetChecked();
	
	// Define tools for all public functions
	for (TFieldIterator<UFunction> FunctionIter(GetClass(), EFieldIteratorFlags::ExcludeSuper); FunctionIter; ++FunctionIter)
	{
		UFunction* Function = *FunctionIter;
		const FName FunctionName = Function->GetFName();
		if (FunctionName == NAME_ExecuteUbergraph
			|| !FunctionIter->HasAnyFunctionFlags(FUNC_Public))
		{
			continue;
		}

		TSharedRef<FModelContextProtocolLibraryTool> Tool = MakeShared<FModelContextProtocolLibraryTool>(this, Function); 
		Tools.Add(Tool);

		ModelContextProtocolModule.AddTool(Tool);
	}

	// Re-register tools on refresh
	ModelContextProtocolModule.OnRefreshTools().AddWeakLambda(this, [this]()
		{
			RegisterTools();
		});
}

void UModelContextProtocolToolLibrary::DeregisterTools()
{
	if (IModelContextProtocolModule* ModelContextProtocolModule = IModelContextProtocolModule::Get())
	{
		for (const TSharedRef<FModelContextProtocolLibraryTool>& Tool : Tools)
		{
			ModelContextProtocolModule->RemoveTool(Tool);
		}

		// Re-register tools on refresh
		ModelContextProtocolModule->OnRefreshTools().RemoveAll(this);
	}
	Tools.Reset();
}

#if WITH_EDITORONLY_DATA
void UModelContextProtocolToolLibrary::CollectFunctionMetaData()
{
	Modify();

	FunctionMetaData.Reset();
	for (TFieldIterator<UFunction> FunctionIter(GetClass(), EFieldIteratorFlags::ExcludeSuper); FunctionIter; ++FunctionIter)
	{
		const UFunction* Function = *FunctionIter;
		const FName FunctionName = Function->GetFName();

		// Skip hidden functions
		if (FunctionName == NAME_ExecuteUbergraph
			|| !FunctionIter->HasAnyFunctionFlags(FUNC_Public))
		{
			continue;
		}

		FJsonSchemaPropertyFilter PropertyFilter(/*CheckFlags*/CPF_Parm | CPF_ReturnParm | CPF_OutParm | CPF_ConstParm);
		FunctionMetaData.Add(FunctionName, UE::ModelContextProtocol::CollectFunctionMetaData(Function, PropertyFilter));
	}
}
#endif // WITH_EDITORONLY_DATA

UModelContextProtocolToolLibraryBlueprint::UModelContextProtocolToolLibraryBlueprint()
{
	BlueprintType = BPTYPE_FunctionLibrary;
	ParentClass = UModelContextProtocolToolLibrary::StaticClass();
#if WITH_EDITORONLY_DATA
	ShouldCookPropertyGuidsValue = EShouldCookBlueprintPropertyGuids::Yes;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
bool UModelContextProtocolToolLibraryBlueprint::AlwaysCompileOnLoad() const
{
	return true;
}
#endif

FModelContextProtocolLibraryTool::FModelContextProtocolLibraryTool(UModelContextProtocolToolLibrary* InLibrary, UFunction* InFunction)
: Library(InLibrary)
, Function(InFunction)
, FunctionParamsContainer(Function)
{
	check(Library);
	check(Function);
}

FString FModelContextProtocolLibraryTool::GetName() const
{
	check(Function);
	return Function->GetName();
}

FString FModelContextProtocolLibraryTool::GetDescription() const
{
	check(Library);
	check(Function);
	if (const FModelContextProtocolFunctionMetaData* FunctionMetaData = Library->FindFunctionMetaData(Function->GetFName()))
	{
		if (FunctionMetaData->Description.IsSet())
		{
			return FunctionMetaData->Description->ToString();
		}
	}

	return FString();
}

TSharedPtr<FJsonObject> FModelContextProtocolLibraryTool::GetInputJsonSchema() const
{
	check(Library);
	check(Function);

	// Return cached?
	if (InputJsonSchema.IsValid())
	{
		return InputJsonSchema;
	}

	const FModelContextProtocolFunctionMetaData* FunctionMetaData = Library->FindFunctionMetaData(Function->GetFName());

	// Build property filter — skip WorldContext param as we auto-set it, and skip return/out params
	TOptional<TSet<FString>> SkipPropertyMemberPaths;
	if (FunctionMetaData && !FunctionMetaData->WorldContext.IsEmpty())
	{
		SkipPropertyMemberPaths.Emplace();
		SkipPropertyMemberPaths->Add(FunctionMetaData->WorldContext);
	}

	FJsonSchemaPropertyFilter PropertyFilter(
		/*CheckFlags*/CPF_Parm | CPF_ConstParm,
		/*SkipFlags*/CPF_ReturnParm | CPF_OutParm,
		/*RequiredPropertyMemberPaths*/{},
		SkipPropertyMemberPaths);

	// Convert cookable metadata to engine format for schema generation
	TOptional<FJsonSchemaEditorMetadata> CachedEditorMetadata;
	if (FunctionMetaData)
	{
		CachedEditorMetadata = UE::ModelContextProtocol::ConvertToCachedEditorMetadata(*FunctionMetaData);
	}

	TSharedPtr<FJsonObject> SchemaResult = FJsonSchemaGenerator::UStructToJsonSchemaObject(
		Function, PropertyFilter, CachedEditorMetadata.IsSet() ? &CachedEditorMetadata.GetValue() : nullptr);

	// UStructToJsonSchemaObject returns JSON-RPC wrapper for UFunctions: {inputSchema, outputSchema}
	// Extract inputSchema
	if (SchemaResult.IsValid())
	{
		const TSharedPtr<FJsonObject>* InputSchemaField;
		if (SchemaResult->TryGetObjectField(TEXT("inputSchema"), InputSchemaField))
		{
			InputJsonSchema = *InputSchemaField;
		}
	}

	return InputJsonSchema;
}

TSharedPtr<FJsonObject> FModelContextProtocolLibraryTool::GetOutputJsonSchema() const
{
	CacheResultTypeInfo();
	
	return OutputJsonSchema;
}

FModelContextProtocolToolResult FModelContextProtocolLibraryTool::Run(const TSharedPtr<FJsonObject>& Params)
{
#if WITH_EDITOR
	const FScopedTransaction Transaction(FText::FromString(FName::NameToDisplayString(GetName(), /*bIsBool*/false)));
#endif // WITH_EDITOR
	
	check(Library);
	check(Function);

	CacheResultTypeInfo();

	// Prepare params
	Function->InitializeStruct(FunctionParamsContainer.GetStructMemory());
	ON_SCOPE_EXIT
	{
		Function->DestroyStruct(FunctionParamsContainer.GetStructMemory());
	};

	// Apply param defaults
	const FModelContextProtocolFunctionMetaData* FunctionMetaData = Library->FindFunctionMetaData(Function->GetFName());
	if (FunctionMetaData)
	{
		for (TFieldIterator<FProperty> ParamPropertyIt(Function); ParamPropertyIt; ++ParamPropertyIt)
		{
			const FProperty* ParamProperty = *ParamPropertyIt;
			FString ParamName = ParamProperty->GetAuthoredName();
			if (const FModelContextProtocolPropertyMetaData* PropertyMetaData = FunctionMetaData->PropertyMetaData.Find(*ParamName);
				PropertyMetaData && PropertyMetaData->DefaultValue.IsSet())
			{
				ParamProperty->ImportText_InContainer(**PropertyMetaData->DefaultValue, FunctionParamsContainer.GetStructMemory(), Library, PPF_None);
			}
		}
	}
	
	if (Params.IsValid())
	{
		// Json Params -> UFunction params container
		FText FailReason;
		if (!FJsonObjectConverter::JsonObjectToUStruct(Params.ToSharedRef(), Function, FunctionParamsContainer.GetStructMemory(), /*CheckFlags*/CPF_Parm | CPF_ConstParm, /*SkipFlags*/CPF_ReturnParm | CPF_OutParm, /*bStrictMode*/false, &FailReason))
		{
			return UE::ModelContextProtocol::MakeErrorResult(FailReason.ToString());
		}
	}

	// WorldContext
	if (FunctionMetaData && !FunctionMetaData->WorldContext.IsEmpty())
	{
		UObject* WorldContextObject = GWorld.GetReference();
		if (FProperty* WorldContextProperty = Function->FindPropertyByName(FName(FunctionMetaData->WorldContext)); ensure(WorldContextProperty))
		{
			if (FObjectPropertyBase* WorldContextObjectProperty = CastField<FObjectPropertyBase>(WorldContextProperty); ensure(WorldContextObjectProperty))
			{
				WorldContextObjectProperty->SetObjectPropertyValue_InContainer(FunctionParamsContainer.GetStructMemory(), WorldContextObject);
			}
		}
	}
	
	{
		// Execute tool
		FEditorScriptExecutionGuard ScriptExecutionGuard;
		Library->ProcessEvent(Function, FunctionParamsContainer.GetStructMemory());
	}

	if (FProperty* ReturnProperty = Function->GetReturnProperty(); ReturnProperty && ensure(Function->ReturnValueOffset != MAX_uint16))
	{
		check(ResultType.IsSet());

		return UE::ModelContextProtocol::GetToolResultFromType(*ResultType, ReturnProperty, FunctionParamsContainer.GetStructMemory(), Function->ReturnValueOffset);
	}

	// No result (void return)
	return FModelContextProtocolToolResult();
}

void FModelContextProtocolLibraryTool::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Library);
	Collector.AddReferencedObject(Function);
}

void FModelContextProtocolLibraryTool::CacheResultTypeInfo() const
{
	check(Library);
	check(Function);

	// Already cached?
	if (ResultType.IsSet())
	{
		return;
	}

	// Determine result type
	ResultType = EModelContextProtocolToolResultType::None;
	OutputJsonSchema.Reset();
	if (FProperty* ReturnProperty = Function->GetReturnProperty(); ReturnProperty && ensure(Function->ReturnValueOffset != MAX_uint16))
	{
		const FModelContextProtocolFunctionMetaData* FunctionMetaData = Library->FindFunctionMetaData(Function->GetFName());

		ResultType = UE::ModelContextProtocol::GetToolResultType(ReturnProperty, FunctionMetaData, OutputJsonSchema);
	}
}

#undef LOCTEXT_NAMESPACE
