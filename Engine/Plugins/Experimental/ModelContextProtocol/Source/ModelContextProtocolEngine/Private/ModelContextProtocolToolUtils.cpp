// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolUtils.h"
#include "ModelContextProtocol.h"
#include "ModelContextProtocolEngineMetaData.h"
#include "ModelContextProtocolEngineToolResults.h"
#include "ModelContextProtocolMetaData.h"
#include "ModelContextProtocolToolResults.h"

#include "Engine/Texture2D.h"
#include "JsonSchema/JsonSchemaGenerator.h"
#include "JsonSchema/JsonSchemaPropertyFilter.h"
#include "Sound/SoundWave.h"
#include "UObject/TextProperty.h"
#include "UObject/Utf8StrProperty.h"

namespace UE::ModelContextProtocol
{
	EModelContextProtocolToolResultType GetToolResultType(FProperty* Property, const FModelContextProtocolFunctionMetaData* MetaData, TSharedPtr<FJsonObject>& OutSchema)
	{
		OutSchema.Reset();

		if (Property)
		{
			// Image result?
			if (const FObjectProperty* ObjectReturnProperty = CastField<FObjectProperty>(Property);
				ObjectReturnProperty && ObjectReturnProperty->PropertyClass == UTexture2D::StaticClass())
			{
				return EModelContextProtocolToolResultType::Image;
			}

			// Audio result?
			if (const FObjectProperty* ObjectReturnProperty = CastField<FObjectProperty>(Property);
				ObjectReturnProperty && ObjectReturnProperty->PropertyClass == USoundWave::StaticClass())
			{
				return EModelContextProtocolToolResultType::Audio;
			}

			// Text result?
			if (Property->IsA<FStrProperty>()
				|| Property->IsA<FTextProperty>()
				|| Property->IsA<FUtf8StrProperty>())
			{
				return EModelContextProtocolToolResultType::Text;
			}

			// We want to convert soft object references to text paths
			if (Property->IsA<FSoftObjectProperty>())
			{
				return EModelContextProtocolToolResultType::Text;
			}

			// Return structured result
			// @todo Support reference output vars
			{
				FJsonSchemaPropertyFilter PropertyFilter;
				TOptional<FJsonSchemaEditorMetadata> CachedEditorMetadata;
				if (MetaData)
				{
					CachedEditorMetadata = UE::ModelContextProtocol::ConvertToCachedEditorMetadata(*MetaData);
				}
				OutSchema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
					Property, PropertyFilter, CachedEditorMetadata.IsSet() ? &CachedEditorMetadata.GetValue() : nullptr);
			}

			// {"result":} object wrapper for POD results?
			if (UE::ModelContextProtocol::bWrapPODResultsInObject && OutSchema.IsValid())
			{
				FString TypeString;

				if (OutSchema->TryGetStringField(TEXT("type"), TypeString))
				{
					if (TypeString != TEXT("object"))
					{
						FJsonDomBuilder::FObject ResultWrapperObjectSchema;
						ResultWrapperObjectSchema.Set(TEXT("type"), TEXT("object"));
						ResultWrapperObjectSchema.Set(TEXT("properties"), FJsonDomBuilder::FObject().Set(UE::ModelContextProtocol::PODWrapperResultPropertyName, OutSchema));
						ResultWrapperObjectSchema.Set(TEXT("required"), FJsonDomBuilder::FArray().Add(FString(UE::ModelContextProtocol::PODWrapperResultPropertyName)));

						OutSchema = ResultWrapperObjectSchema.AsJsonObject();
					}
				}
			}

			return EModelContextProtocolToolResultType::StructuredContent;
		}

		return EModelContextProtocolToolResultType::None;
	}

	FModelContextProtocolToolResult GetToolResultFromType(EModelContextProtocolToolResultType ResultType, FProperty* Property, void* Container, uint16 ReturnValueOffset)
	{
		if (ResultType == EModelContextProtocolToolResultType::Image)
		{
			if (FObjectProperty* ObjectReturnProperty = CastField<FObjectProperty>(Property); ensure(ObjectReturnProperty) && ensure(ObjectReturnProperty->PropertyClass == UTexture2D::StaticClass()))
			{
				if (const UObject* ObjectResult = ObjectReturnProperty->GetObjectPropertyValue_InContainer(Container))
				{
					if (const UTexture2D* TextureObjectResult = Cast<UTexture2D>(ObjectResult); ensure(TextureObjectResult))
					{
						return UE::ModelContextProtocol::MakeImageResult(TextureObjectResult);
					}
				}
			}
		}
		else if (ResultType == EModelContextProtocolToolResultType::Audio)
		{
			if (FObjectProperty* ObjectReturnProperty = CastField<FObjectProperty>(Property); ensure(ObjectReturnProperty) && ensure(ObjectReturnProperty->PropertyClass == USoundWave::StaticClass()))
			{
				if (UObject* ObjectResult = ObjectReturnProperty->GetObjectPropertyValue_InContainer(Container))
				{
					if (USoundWave* SoundObjectResult = Cast<USoundWave>(ObjectResult); ensure(SoundObjectResult))
					{
						return UE::ModelContextProtocol::MakeAudioResult(SoundObjectResult);
					}
				}
			}
		}
		else if (ResultType == EModelContextProtocolToolResultType::Text)
		{
			// Export text with PPF_PropertyWindow PortFlags to ensure FText is exported as simple string
			FString StringResult;
			Property->ExportTextItem_InContainer(StringResult, Container, /*DefaultValue*/nullptr, /*Parent*/nullptr, PPF_PropertyWindow);

			return UE::ModelContextProtocol::MakeTextResult(StringResult);
		}
		else if (ResultType == EModelContextProtocolToolResultType::StructuredContent)
		{
			// Return structured result
			// @todo Support for OutParm's using UStructToJson with OutParm CheckFlags on FunctionParamsContainer
			return UE::ModelContextProtocol::MakeStructuredContentResult(Property, (uint8*)Container + ReturnValueOffset);
		}
		else
		{
			ensureMsgf(false, TEXT("Unhandled result type %s"), *StaticEnum<EModelContextProtocolToolResultType>()->GetValueAsString(ResultType));
		}

		return FModelContextProtocolToolResult();
	}
};