// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocolClientConfig.generated.h"

/** Supported AI coding clients for MCP configuration file generation. */
UENUM()
enum class EModelContextProtocolClient : uint8
{
	/** Claude Code: `.mcp.json` in project root */
	ClaudeCode,
	/** Cursor: `.cursor/mcp.json` in project root */
	Cursor,
	/** VS Code / Copilot: `.vscode/mcp.json` in project root */
	VSCode,
	/** Gemini CLI: `.gemini/settings.json` in project root */
	Gemini,
	/** Codex CLI: `.codex/config.toml` in project root (TOML, write-once) */
	Codex
};

namespace UE::ModelContextProtocol
{
	/**
	 * Generates an MCP client configuration file.
	 *
	 * JSON formats (ClaudeCode, Cursor, VSCode, Gemini) are created or updated, preserving existing entries.
	 * TOML format (Codex) is written once; logs error and returns false if file already exists.
	 *
	 * @param Client The target AI client.
	 * @param Port The MCP server port number.
	 * @param UrlPath The MCP server URL path (e.g., "/mcp").
	 * @param BaseDirectory The root directory for config files. Defaults to FPaths::RootDir() in source builds or FPaths::ProjectDir() in installed builds if empty.
	 * @return true if the file was written successfully.
	 * @see EModelContextProtocolClient
	 */
	MODELCONTEXTPROTOCOLENGINE_API bool WriteClientConfiguration(EModelContextProtocolClient Client, uint32 Port, const FString& UrlPath, const FString& BaseDirectory = FString());

	/**
	 * Generates MCP client configuration files for all supported clients.
	 *
	 * @param Port The MCP server port number.
	 * @param UrlPath The MCP server URL path (e.g., "/mcp").
	 * @param BaseDirectory The root directory for config files. Defaults to FPaths::RootDir() in source builds or FPaths::ProjectDir() in installed builds if empty.
	 * @return Number of files successfully written.
	 */
	MODELCONTEXTPROTOCOLENGINE_API int32 WriteAllClientConfigurations(uint32 Port, const FString& UrlPath, const FString& BaseDirectory = FString());
}
