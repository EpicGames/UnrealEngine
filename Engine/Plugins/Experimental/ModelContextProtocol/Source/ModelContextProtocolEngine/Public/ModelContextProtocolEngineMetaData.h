// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocolMetaData.h"

struct FJsonSchemaEditorMetadata;
struct FJsonSchemaPropertyFilter;

namespace UE::ModelContextProtocol
{
#if WITH_EDITORONLY_DATA
	/**
	 * Collects editor-only metadata for a UFunction, storing it in cookable form.
	 *
	 * @param Function The function to collect metadata for.
	 * @param PropertyFilter Property filter for schema generation.
	 * @return Collected function metadata.
	 */
	MODELCONTEXTPROTOCOLENGINE_API FModelContextProtocolFunctionMetaData CollectFunctionMetaData(const UFunction* Function, const FJsonSchemaPropertyFilter& PropertyFilter);
#endif

	/**
	 * Converts cookable function metadata back into FJsonSchemaEditorMetadata for passing to FJsonSchemaGenerator.
	 *
	 * @param FunctionMetaData The cookable metadata to convert.
	 * @return Engine editor metadata suitable for schema generation.
	 */
	MODELCONTEXTPROTOCOLENGINE_API FJsonSchemaEditorMetadata ConvertToCachedEditorMetadata(const FModelContextProtocolFunctionMetaData& FunctionMetaData);
}
