// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocolToolResults.h"

class USoundWave;
class UTexture2D;
struct FColor;
struct FImageView;

namespace UE::ModelContextProtocol
{
	// @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools#image-content
	MODELCONTEXTPROTOCOLENGINE_API FModelContextProtocolToolResult MakeImageResult(const UTexture2D* Texture, const TCHAR* ToFormatExtension = TEXT("jpeg"), EModelContextProtocolAudience Audience = EModelContextProtocolAudience::All);
	MODELCONTEXTPROTOCOLENGINE_API FModelContextProtocolToolResult MakeImageResult(const FImageView& Image, const TCHAR* ToFormatExtension = TEXT("jpeg"), EModelContextProtocolAudience Audience = EModelContextProtocolAudience::All);

	// @see https://modelcontextprotocol.io/specification/2025-06-18/server/tools#audio-content
	MODELCONTEXTPROTOCOLENGINE_API FModelContextProtocolToolResult MakeAudioResult(USoundWave* SoundWave);
}
