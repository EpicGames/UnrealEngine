// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "ModelContextProtocol.h"

#include "ModelContextProtocolSettings.generated.h"

namespace UE::ModelContextProtocol
{
	MODELCONTEXTPROTOCOLENGINE_API FString GetServerUrlPath();
	MODELCONTEXTPROTOCOLENGINE_API uint32 GetServerPortNumber();
	MODELCONTEXTPROTOCOLENGINE_API bool ShouldAutoStartServer();
}

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, meta=(DisplayName="Model Context Protocol"))
class UModelContextProtocolSettings : public UDeveloperSettings
{
	GENERATED_BODY()
public:

	/** The URL base path to serve from e.g: "/mcp" -> http://localhost/mcp */
	UPROPERTY(Config, EditAnywhere, Category="Server")
	FString ServerUrlPath = UE::ModelContextProtocol::DefaultServerUrlPath;

	/** If InOutPath fails FHttpPath::IsValidPath, replaces it with UE::ModelContextProtocol::DefaultServerUrlPath and logs a warning. Exposed so the editor-side guard and direct tests can share the same logic. */
	static MODELCONTEXTPROTOCOLENGINE_API void EnforceValidServerUrlPath(FString& InOutPath);

#if WITH_EDITOR
	//~ Begin UObject Interface
	MODELCONTEXTPROTOCOLENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
#endif

	/** The port number to serve from e.g: 8000 -> http://localhost:8000/mcp */
	UPROPERTY(Config, EditAnywhere, Category="Server")
	uint32 ServerPortNumber = 8000;

	/** If true, the HTTP server route will be automatically registered and HTTP listeners started during module startup. */
	UPROPERTY(Config, EditAnywhere, Category="Server")
	bool bAutoStartServer = false;

	/** If true, tools/list returns only list_toolsets, describe_toolset, and call_tool. The LLM discovers toolset tools on demand and dispatches them through call_tool without ever registering them as native MCP tools. If false, every toolset tool is registered natively at startup. */
	UPROPERTY(Config, EditAnywhere, Category="Tools")
	bool bEnableToolSearch = true;
};

