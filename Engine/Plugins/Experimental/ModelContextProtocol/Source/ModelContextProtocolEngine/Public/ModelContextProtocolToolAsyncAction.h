// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/CancellableAsyncAction.h"
#include "IModelContextProtocolTool.h"
#include "ModelContextProtocolMetaData.h"
#include "ModelContextProtocolSession.h"
#include "UObject/StructOnScope.h"

#include "ModelContextProtocolToolAsyncAction.generated.h"

struct FModelContextProtocolAsyncActionTool;

#define UE_API MODELCONTEXTPROTOCOLENGINE_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnModelContextProtocolAsyncToolComplete, UModelContextProtocolToolAsyncAction* /* AsyncAction */);

/**
 * @deprecated Use UToolsetDefinition (ToolsetRegistry plugin) instead. UModelContextProtocolToolAsyncAction is maintained as a legacy
 *             fallback and will be removed in a future release.
 */
UCLASS(Abstract, MinimalAPI, meta=(DeprecatedNode, DeprecationMessage="Use UToolsetDefinition (ToolsetRegistry plugin) instead."))
class UModelContextProtocolToolAsyncAction : public UCancellableAsyncAction
{
	GENERATED_BODY()

public:
	UE_API virtual void SetReadyToDestroy() override;

	virtual FName GetToolFunctionName() const { return NAME_None; }
	virtual FName GetToolResultPropertyName() const { return NAME_None; }

	const FModelContextProtocolFunctionMetaData* GetToolFunctionMetaData() const { return &ToolFunctionMetaData; }
	const FModelContextProtocolFunctionMetaData* GetToolResultMetaData() const { return &ToolResultMetaData; }

	bool ShouldAutoRegisterTool() const { return bAutoRegisterTool; }

	UE_API void RegisterTool();
	UE_API void DeregisterTool();

	//~ Begin UObject API
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void FinishDestroy() override;
	//~ End UObject API

	FOnModelContextProtocolAsyncToolComplete OnAsyncToolComplete;

	FModelContextProtocolToolRequestId ToolRequestId;

protected:
	/**
	 * If true, automatically registers on module load.
	 * If false, RegisterTool must manually be called.
	 */ 
	UPROPERTY(EditDefaultsOnly, Category = "Model Context Protocol")
	bool bAutoRegisterTool = true;

	UPROPERTY(VisibleDefaultsOnly, Category = "Model Context Protocol")
	FModelContextProtocolFunctionMetaData ToolFunctionMetaData;

	UPROPERTY(VisibleDefaultsOnly, Category = "Model Context Protocol")
	FModelContextProtocolFunctionMetaData ToolResultMetaData;

	TSharedPtr<FModelContextProtocolAsyncActionTool> Tool;

	IModelContextProtocolTool::FResultCallback OnComplete;

#if WITH_EDITORONLY_DATA
	void CollectToolMetaData();
#endif
};

/**
 * IModelContextProtocolTool implementation, automatically derived from reflected UModelContextProtocolToolAsyncAction, using UFUNCTION comment
 * to derived tool & parameter descriptions.
 * @see IModelContextProtocolTool
 * @see UModelContextProtocolToolAsyncAction
 */
struct FModelContextProtocolAsyncActionTool: public IModelContextProtocolTool
{
public:
	FModelContextProtocolAsyncActionTool(UModelContextProtocolToolAsyncAction* InAsyncAction, UFunction* InFunction);

	//~ Begin IModelContextProtocolTool API
	virtual FString GetName() const override;
	virtual FString GetDescription() const override;
	virtual TSharedPtr<FJsonObject> GetInputJsonSchema() const override;
	virtual TSharedPtr<FJsonObject> GetOutputJsonSchema() const override;
	virtual void RunAsync(const FModelContextProtocolToolRequestId& RequestId, const TSharedPtr<FJsonObject>& Params, const FResultCallback& OnComplete) override;
	virtual void CancelAsync(const FModelContextProtocolToolRequestId& RequestId) override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	//~ End IModelContextProtocolTool API

protected:
	// UModelContextProtocolToolAsyncAction subclass CDO
	TObjectPtr<UModelContextProtocolToolAsyncAction> AsyncAction;
	TObjectPtr<UFunction> Function;
	TArray<TObjectPtr<UModelContextProtocolToolAsyncAction>> InProgressActions;

	mutable TSharedPtr<FJsonObject> InputJsonSchema;
	mutable TOptional<EModelContextProtocolToolResultType> ResultType;
	mutable TSharedPtr<FJsonObject> OutputJsonSchema;
	FStructOnScope FunctionParamsContainer;

	void CacheResultTypeInfo() const;
};

#undef UE_API
