// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IModelContextProtocolTool.h"

#include "Engine/Blueprint.h"
#include "ModelContextProtocolMetaData.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/StructOnScope.h"

#include "ModelContextProtocolToolLibrary.generated.h"

#define UE_API MODELCONTEXTPROTOCOLENGINE_API

class UModelContextProtocolToolLibrary;
struct FModelContextProtocolLibraryTool;

/**
 * Automatically registers an FModelContextProtocolLibraryTool for each public UFUNCTION on module load, using cached / cooked meta-data.
 *
 * UBlueprintFunctionLibrary specialization allows caching and cooking otherwise-editor-only UFUNCTION descriptions and parameter meta-data.
 *
 * Can be subclassed in either C++ or Blueprints.
 *
 * For Blueprints:	Create via Content Browser -> Add -> MCP Tool Library, then simply define public functions with a doxygen-style
 *					function tooltip.
 *
 * @deprecated Use UToolsetDefinition (ToolsetRegistry plugin) instead. UModelContextProtocolToolLibrary is maintained as a legacy fallback
 *             and will be removed in a future release.
 */
UCLASS(Abstract, MinimalAPI, meta=(DeprecatedNode, DeprecationMessage="Use UToolsetDefinition (ToolsetRegistry plugin) instead."))
class UModelContextProtocolToolLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	const TMap<FName, FModelContextProtocolFunctionMetaData>& GetFunctionMetaData() const
	{
		return FunctionMetaData;
	}

	const FModelContextProtocolFunctionMetaData* FindFunctionMetaData(const FName& FunctionName) const
	{
		return FunctionMetaData.Find(FunctionName);
	}

	bool ShouldAutoRegisterTools() const { return bAutoRegisterTools; }

	UE_API void RegisterTools();
	UE_API void DeregisterTools();
	
	//~ Begin UObject API
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void FinishDestroy() override;
	//~ End UObject API
	
protected:

	/**
	 * If true, automatically registers an FModelContextProtocolLibraryTool for each public UFUNCTION on module load.
	 * If false, RegisterTools must manually be called.
	 */ 
	UPROPERTY(EditDefaultsOnly, Category = "Model Context Protocol")
	bool bAutoRegisterTools = true;

	/** Per-UFUNCTION cached meta-data to allow meta-data use on cooked platforms */ 
	UPROPERTY(VisibleDefaultsOnly, Category = "Model Context Protocol")
	TMap<FName, FModelContextProtocolFunctionMetaData> FunctionMetaData;

	TArray<TSharedRef<FModelContextProtocolLibraryTool>> Tools;

#if WITH_EDITOR
	UE_API virtual void PostCDOCompiled(const FPostCDOCompiledContext& Context) override;
#endif
#if WITH_EDITORONLY_DATA
	void CollectFunctionMetaData();
#endif
};

UCLASS(MinimalAPI)
class UModelContextProtocolToolLibraryBlueprint : public UBlueprint
{
	GENERATED_BODY()
public:
	UModelContextProtocolToolLibraryBlueprint();

	//~ Begin UBlueprint API
#if WITH_EDITOR
	virtual bool AlwaysCompileOnLoad() const override;
#endif // WITH_EDITOR
	//~ End UBlueprint API
};

/**
 * IModelContextProtocolTool implementation, automatically derived from reflected UModelContextProtocolToolLibrary functions, using UFUNCTION comment
 * to derived tool & parameter descriptions.
 * @see IModelContextProtocolTool
 * @see UModelContextProtocolToolLibrary
 */
struct FModelContextProtocolLibraryTool : public IModelContextProtocolTool
{
public:
	FModelContextProtocolLibraryTool(UModelContextProtocolToolLibrary* InLibrary, UFunction* InFunction);
	
	//~ Begin IModelContextProtocolTool API
	virtual FString GetName() const override;
	virtual FString GetDescription() const override; 
	virtual TSharedPtr<FJsonObject> GetInputJsonSchema() const override;
	virtual TSharedPtr<FJsonObject> GetOutputJsonSchema() const override;
	virtual FModelContextProtocolToolResult Run(const TSharedPtr<FJsonObject>& Params) override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	//~ End IModelContextProtocolTool API
protected:
	// UModelContextProtocolToolLibrary subclass CDO
	TObjectPtr<UModelContextProtocolToolLibrary> Library;
	TObjectPtr<UFunction> Function;
	mutable TSharedPtr<FJsonObject> InputJsonSchema;
	mutable TOptional<EModelContextProtocolToolResultType> ResultType;
	mutable TSharedPtr<FJsonObject> OutputJsonSchema;
	FStructOnScope FunctionParamsContainer;

	void CacheResultTypeInfo() const;
};

#undef UE_API
