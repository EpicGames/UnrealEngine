// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocolToolResults.h"

class FProperty;
struct FModelContextProtocolFunctionMetaData;

#define UE_API MODELCONTEXTPROTOCOLENGINE_API

namespace UE::ModelContextProtocol
{
	UE_API EModelContextProtocolToolResultType GetToolResultType(FProperty* Property, const FModelContextProtocolFunctionMetaData* MetaData, TSharedPtr<FJsonObject>& OutSchema);
	UE_API FModelContextProtocolToolResult GetToolResultFromType(EModelContextProtocolToolResultType ResultType, FProperty* Property, void* Container, uint16 ReturnValueOffset);
};

#undef UE_API