// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FModelContextProtocolEngineTestsModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FModelContextProtocolEngineTestsModule, ModelContextProtocolEngineTests);
