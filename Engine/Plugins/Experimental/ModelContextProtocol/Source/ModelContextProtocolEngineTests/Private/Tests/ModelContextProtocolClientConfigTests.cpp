// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolClientConfig.h"
#include "ModelContextProtocol.h"

#include "Dom/JsonObject.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolClientConfigTests, "AI.ModelContextProtocol.ClientConfig", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	FString TestDirectory;

	TSharedPtr<FJsonObject> ReadJsonFile(const FString& FilePath)
	{
		FString Content;
		if (!FFileHelper::LoadFileToString(Content, *FilePath))
		{
			return nullptr;
		}
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Content);
		FJsonSerializer::Deserialize(Reader, JsonObject);
		return JsonObject;
	}
END_DEFINE_SPEC(FModelContextProtocolClientConfigTests)

void FModelContextProtocolClientConfigTests::Define()
{
	BeforeEach([this]()
	{
		TestDirectory = FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("Temp"), TEXT("MCPClientConfigTests"));
		IFileManager::Get().MakeDirectory(*TestDirectory, true);
	});

	AfterEach([this]()
	{
		IFileManager::Get().DeleteDirectory(*TestDirectory, false, true);
	});

	Describe("WriteClientConfiguration", [this]()
	{
		It("should write ClaudeCode .mcp.json with correct format", [this]()
		{
			TestTrue("Should write successfully", UE::ModelContextProtocol::WriteClientConfiguration(EModelContextProtocolClient::ClaudeCode, 8000, TEXT("/mcp"), TestDirectory));

			const FString FilePath = FPaths::Combine(TestDirectory, TEXT(".mcp.json"));
			TSharedPtr<FJsonObject> Root = ReadJsonFile(FilePath);
			if (!TestTrue("Should parse JSON", Root.IsValid())) { return; }

			const TSharedPtr<FJsonObject>* Servers = nullptr;
			if (!TestTrue("Should have mcpServers", Root->TryGetObjectField(TEXT("mcpServers"), Servers))) { return; }

			const TSharedPtr<FJsonObject>* Entry = nullptr;
			if (!TestTrue("Should have unreal-mcp entry", (*Servers)->TryGetObjectField(UE::ModelContextProtocol::DefaultServerName, Entry))) { return; }

			FString Type, Url;
			TestTrue("Should have type field", (*Entry)->TryGetStringField(TEXT("type"), Type));
			TestEqual("Type should be http", Type, TEXT("http"));
			TestTrue("Should have url field", (*Entry)->TryGetStringField(TEXT("url"), Url));
			TestEqual("URL should match", Url, TEXT("http://127.0.0.1:8000/mcp"));
		});

		It("should write Cursor .cursor/mcp.json with correct format", [this]()
		{
			TestTrue("Should write successfully", UE::ModelContextProtocol::WriteClientConfiguration(EModelContextProtocolClient::Cursor, 9000, TEXT("/mcp"), TestDirectory));

			const FString FilePath = FPaths::Combine(TestDirectory, TEXT(".cursor/mcp.json"));
			TSharedPtr<FJsonObject> Root = ReadJsonFile(FilePath);
			if (!TestTrue("Should parse JSON", Root.IsValid())) { return; }

			const TSharedPtr<FJsonObject>* Servers = nullptr;
			if (!TestTrue("Should have mcpServers", Root->TryGetObjectField(TEXT("mcpServers"), Servers))) { return; }

			const TSharedPtr<FJsonObject>* Entry = nullptr;
			if (!TestTrue("Should have unreal-mcp entry", (*Servers)->TryGetObjectField(UE::ModelContextProtocol::DefaultServerName, Entry))) { return; }

			FString Url;
			TestTrue("Should have url field", (*Entry)->TryGetStringField(TEXT("url"), Url));
			TestEqual("URL should match", Url, TEXT("http://127.0.0.1:9000/mcp"));
			TestFalse("Should NOT have type field", (*Entry)->HasField(TEXT("type")));
		});

		It("should write VSCode .vscode/mcp.json with servers root key", [this]()
		{
			TestTrue("Should write successfully", UE::ModelContextProtocol::WriteClientConfiguration(EModelContextProtocolClient::VSCode, 8000, TEXT("/mcp"), TestDirectory));

			const FString FilePath = FPaths::Combine(TestDirectory, TEXT(".vscode/mcp.json"));
			TSharedPtr<FJsonObject> Root = ReadJsonFile(FilePath);
			if (!TestTrue("Should parse JSON", Root.IsValid())) { return; }

			const TSharedPtr<FJsonObject>* Servers = nullptr;
			TestTrue("Should have servers (not mcpServers)", Root->TryGetObjectField(TEXT("servers"), Servers));
			TestFalse("Should NOT have mcpServers", Root->HasField(TEXT("mcpServers")));
		});

		It("should write Gemini .gemini/settings.json with httpUrl field", [this]()
		{
			TestTrue("Should write successfully", UE::ModelContextProtocol::WriteClientConfiguration(EModelContextProtocolClient::Gemini, 8000, TEXT("/mcp"), TestDirectory));

			const FString FilePath = FPaths::Combine(TestDirectory, TEXT(".gemini/settings.json"));
			TSharedPtr<FJsonObject> Root = ReadJsonFile(FilePath);
			if (!TestTrue("Should parse JSON", Root.IsValid())) { return; }

			const TSharedPtr<FJsonObject>* Servers = nullptr;
			if (!TestTrue("Should have mcpServers", Root->TryGetObjectField(TEXT("mcpServers"), Servers))) { return; }

			const TSharedPtr<FJsonObject>* Entry = nullptr;
			if (!TestTrue("Should have unreal-mcp entry", (*Servers)->TryGetObjectField(UE::ModelContextProtocol::DefaultServerName, Entry))) { return; }

			FString HttpUrl;
			TestTrue("Should have httpUrl field", (*Entry)->TryGetStringField(TEXT("httpUrl"), HttpUrl));
			TestEqual("httpUrl should match", HttpUrl, TEXT("http://127.0.0.1:8000/mcp"));
			TestFalse("Should NOT have url field", (*Entry)->HasField(TEXT("url")));
		});

		It("should write Codex .codex/config.toml with correct format", [this]()
		{
			TestTrue("Should write successfully", UE::ModelContextProtocol::WriteClientConfiguration(EModelContextProtocolClient::Codex, 8000, TEXT("/mcp"), TestDirectory));

			const FString FilePath = FPaths::Combine(TestDirectory, TEXT(".codex/config.toml"));
			FString Content;
			if (!TestTrue("Should read file", FFileHelper::LoadFileToString(Content, *FilePath))) { return; }

			TestTrue("Should contain mcp_servers section", Content.Contains(TEXT("[mcp_servers.unreal-mcp]")));
			TestTrue("Should contain url", Content.Contains(TEXT("url = \"http://127.0.0.1:8000/mcp\"")));
		});

		It("should refuse to overwrite existing Codex config", [this]()
		{
			// Write once
			UE::ModelContextProtocol::WriteClientConfiguration(EModelContextProtocolClient::Codex, 8000, TEXT("/mcp"), TestDirectory);

			// Attempt overwrite
			AddExpectedError(TEXT("already exists"), EAutomationExpectedErrorFlags::Contains);
			TestFalse("Should refuse to overwrite", UE::ModelContextProtocol::WriteClientConfiguration(EModelContextProtocolClient::Codex, 9000, TEXT("/mcp"), TestDirectory));
		});

		It("should preserve existing entries when updating JSON config", [this]()
		{
			const FString FilePath = FPaths::Combine(TestDirectory, TEXT(".mcp.json"));

			// Write a pre-existing entry
			TSharedPtr<FJsonObject> Root = MakeShared<FJsonObject>();
			TSharedPtr<FJsonObject> Servers = MakeShared<FJsonObject>();
			TSharedPtr<FJsonObject> ExistingEntry = MakeShared<FJsonObject>();
			ExistingEntry->SetStringField(TEXT("command"), TEXT("my-server"));
			Servers->SetObjectField(TEXT("my-existing-server"), ExistingEntry);
			Root->SetObjectField(TEXT("mcpServers"), Servers);

			FString OutputString;
			TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
			FJsonSerializer::Serialize(Root.ToSharedRef(), Writer);
			FFileHelper::SaveStringToFile(OutputString, *FilePath);

			// Now write the MCP config
			TestTrue("Should write successfully", UE::ModelContextProtocol::WriteClientConfiguration(EModelContextProtocolClient::ClaudeCode, 8000, TEXT("/mcp"), TestDirectory));

			TSharedPtr<FJsonObject> UpdatedRoot = ReadJsonFile(FilePath);
			if (!TestTrue("Should parse updated JSON", UpdatedRoot.IsValid())) { return; }

			const TSharedPtr<FJsonObject>* UpdatedServers = nullptr;
			if (!TestTrue("Should have mcpServers", UpdatedRoot->TryGetObjectField(TEXT("mcpServers"), UpdatedServers))) { return; }

			TestTrue("Should still have existing server", (*UpdatedServers)->HasField(TEXT("my-existing-server")));
			TestTrue("Should also have unreal-mcp", (*UpdatedServers)->HasField(UE::ModelContextProtocol::DefaultServerName));
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
