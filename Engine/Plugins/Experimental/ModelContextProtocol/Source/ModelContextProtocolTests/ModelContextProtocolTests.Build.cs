// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelContextProtocolTests : ModuleRules
{
	public ModelContextProtocolTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Analytics",
				"AnalyticsET",
				"Json",
				"JsonUtilities",
				"HTTPServer",
				"HTTP",
				"ModelContextProtocol",
				"ModelContextProtocolEngine"
			});

		PrivateIncludePaths.Add(System.IO.Path.Combine(ModuleDirectory, "..", "ModelContextProtocol", "Private", "Tests"));
	}
}
