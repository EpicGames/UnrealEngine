// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsET.h"
#include "AnalyticsEventAttribute.h"
#include "IAnalyticsProviderET.h"

/**
 * Minimal IAnalyticsProviderET mock that captures recorded events.
 * Only RecordEvent is meaningful; all other interface methods are stubbed.
 */
class FMockAnalyticsProviderET : public IAnalyticsProviderET
{
public:
	struct FRecordedEvent
	{
		FString EventName;
		TArray<FAnalyticsEventAttribute> Attributes;
	};

	TArray<FRecordedEvent> RecordedEvents;

	const FAnalyticsEventAttribute* FindAttribute(const FRecordedEvent& Event, const FString& AttributeName) const
	{
		return Event.Attributes.FindByPredicate([&AttributeName](const FAnalyticsEventAttribute& Attribute)
			{
				return Attribute.GetName() == AttributeName;
			});
	}

	//~ IAnalyticsProviderET
	virtual void SetAppID(FString&&) override {}
	virtual void SetAppVersion(FString&&) override {}
	virtual bool StartSession(FString, const TArray<FAnalyticsEventAttribute>&) override { return true; }
	virtual bool ShouldRecordEvent(const FString&) const override { return true; }

	virtual void RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override
	{
		RecordedEvents.Add({MoveTemp(EventName), Attributes});
	}

	virtual void RecordEvent(FString&& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, EAnalyticsRecordEventMode) override
	{
		RecordEvent(MoveTemp(EventName), Attributes);
	}

	virtual void SetUrlDomain(const FString&, const TArray<FString>&) override {}
	virtual void SetUrlPath(const FString&) override {}
	virtual void SetHeader(const FString&, const FString&) override {}
	virtual void SetEventCallback(const OnEventRecorded&) override {}
	virtual void BlockUntilFlushed(float) override {}
	virtual const FAnalyticsET::Config& GetConfig() const override { return Config; }
	virtual void SetShouldRecordEventFunc(const ShouldRecordEventFunction&) override {}
	virtual FOnPreAnalyticsEventProcessed& OnPreAnalyticsEventProcessed() override { return PreProcessedDelegate; }
	virtual FOnAnalyticsEventQueued& OnAnalyticsEventQueued() override { return QueuedDelegate; }

	//~ IAnalyticsProvider
	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>&) override { return true; }
	virtual void EndSession() override {}
	virtual FString GetSessionID() const override { return FString(); }
	virtual bool SetSessionID(const FString&) override { return true; }
	virtual void FlushEvents() override {}
	virtual void SetUserID(const FString&) override {}
	virtual FString GetUserID() const override { return FString(); }
	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&&) override {}
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override { return {}; }
	virtual int32 GetDefaultEventAttributeCount() const override { return 0; }
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int) const override { return FAnalyticsEventAttribute(FString(), FString()); }

private:
	FAnalyticsET::Config Config;
	FOnPreAnalyticsEventProcessed PreProcessedDelegate;
	FOnAnalyticsEventQueued QueuedDelegate;
};
