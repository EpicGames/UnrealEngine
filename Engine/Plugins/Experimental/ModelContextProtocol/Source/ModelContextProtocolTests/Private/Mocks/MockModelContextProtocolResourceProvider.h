// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IModelContextProtocolResourceProvider.h"
#include "ModelContextProtocolResources.h"

struct FMockModelContextProtocolResourceProvider : IModelContextProtocolResourceProvider
{
	TArray<FModelContextProtocolResourceDescriptor> Descriptors;
	TMap<FString, FModelContextProtocolResource> Resources;

	bool bListResourcesCalled = false;
	bool bReadResourceCalled = false;
	FString LastReadUri;

	virtual void ListResources(FModelContextProtocolResourceDescriptorList& OutResourceDescriptors) const override
	{
		const_cast<FMockModelContextProtocolResourceProvider*>(this)->bListResourcesCalled = true;
		for (const FModelContextProtocolResourceDescriptor& Descriptor : Descriptors)
		{
			OutResourceDescriptors.Add(Descriptor, ConstCastSharedRef<const IModelContextProtocolResourceProvider>(this->AsShared()));
		}
	}

	virtual TValueOrError<FModelContextProtocolResource, FString> ReadResource(const FString& Uri) const override
	{
		const_cast<FMockModelContextProtocolResourceProvider*>(this)->bReadResourceCalled = true;
		const_cast<FMockModelContextProtocolResourceProvider*>(this)->LastReadUri = Uri;
		if (const FModelContextProtocolResource* Resource = Resources.Find(Uri))
		{
			return MakeValue(*Resource);
		}
		return MakeError(FString::Printf(TEXT("Resource not found: %s"), *Uri));
	}

	/** Add a text resource with a fully described descriptor. */
	void AddTextResource(const FString& Uri, const FString& Content, const TOptional<FString>& Name = {}, const TOptional<FString>& Title = {}, const TOptional<FString>& Description = {}, const TOptional<FString>& MimeType = {})
	{
		Descriptors.Emplace(Uri, Name, Title, Description, MimeType);
		Resources.Add(Uri, FModelContextProtocolResource(Uri, FString(Content), Name, Title, MimeType));
	}

	/** Add a binary resource with a fully described descriptor. */
	void AddBinaryResource(const FString& Uri, const TArray<uint8>& BlobContent, const TOptional<FString>& Name = {}, const TOptional<FString>& Title = {}, const TOptional<FString>& Description = {}, const TOptional<FString>& MimeType = {})
	{
		Descriptors.Emplace(Uri, Name, Title, Description, MimeType);
		Resources.Add(Uri, FModelContextProtocolResource(Uri, TArrayView<uint8>(const_cast<TArray<uint8>&>(BlobContent)), Name, Title, MimeType));
	}
};
