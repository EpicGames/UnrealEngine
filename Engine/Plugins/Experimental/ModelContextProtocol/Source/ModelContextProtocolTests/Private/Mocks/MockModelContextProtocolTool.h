// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IModelContextProtocolTool.h"
#include "ModelContextProtocolSession.h"

struct FMockModelContextProtocolTool : IModelContextProtocolTool
{
	FString Name = TEXT("test_mock_tool");
	FString Description = TEXT("A test mock tool for testing");
	TSharedPtr<FJsonObject> InputSchema;
	TSharedPtr<FJsonObject> OutputSchema;

	TFunction<FModelContextProtocolToolResult(const TSharedPtr<FJsonObject>&)> RunFunction;

	bool bRunCalled = false;
	bool bRunAsyncCalled = false;
	bool bCancelAsyncCalled = false;
	TSharedPtr<FJsonObject> LastRunParams;
	FModelContextProtocolToolRequestId LastRunAsyncRequestId;
	FModelContextProtocolToolRequestId LastCancelledRequestId;

	virtual FString GetName() const override { return Name; }
	virtual FString GetDescription() const override { return Description; }
	virtual TSharedPtr<FJsonObject> GetInputJsonSchema() const override { return InputSchema; }
	virtual TSharedPtr<FJsonObject> GetOutputJsonSchema() const override { return OutputSchema; }

	virtual FModelContextProtocolToolResult Run(const TSharedPtr<FJsonObject>& Params) override
	{
		bRunCalled = true;
		LastRunParams = Params;
		if (RunFunction)
		{
			return RunFunction(Params);
		}
		return UE::ModelContextProtocol::MakeTextResult(TEXT("mock result"));
	}

	virtual void RunAsync(const FModelContextProtocolToolRequestId& RequestId, const TSharedPtr<FJsonObject>& Params, const FResultCallback& OnComplete) override
	{
		bRunAsyncCalled = true;
		LastRunAsyncRequestId = RequestId;
		IModelContextProtocolTool::RunAsync(RequestId, Params, OnComplete);
	}

	virtual void CancelAsync(const FModelContextProtocolToolRequestId& RequestId) override
	{
		bCancelAsyncCalled = true;
		LastCancelledRequestId = RequestId;
	}

	/** Create a JSON Schema object describing simple tool parameters. */
	static TSharedRef<FJsonObject> MakeTestInputSchema(const TMap<FString, FString>& Properties, const TArray<FString>& RequiredProperties = {})
	{
		TSharedRef<FJsonObject> Schema = MakeShared<FJsonObject>();
		Schema->SetStringField(TEXT("type"), TEXT("object"));

		TSharedRef<FJsonObject> PropertiesObject = MakeShared<FJsonObject>();
		for (const TPair<FString, FString>& Property : Properties)
		{
			TSharedRef<FJsonObject> PropertyObject = MakeShared<FJsonObject>();
			PropertyObject->SetStringField(TEXT("type"), Property.Value);
			PropertiesObject->SetObjectField(Property.Key, PropertyObject);
		}
		Schema->SetObjectField(TEXT("properties"), PropertiesObject);

		if (RequiredProperties.Num() > 0)
		{
			TArray<TSharedPtr<FJsonValue>> RequiredArray;
			for (const FString& RequiredProperty : RequiredProperties)
			{
				RequiredArray.Add(MakeShared<FJsonValueString>(RequiredProperty));
			}
			Schema->SetArrayField(TEXT("required"), RequiredArray);
		}

		return Schema;
	}
};
