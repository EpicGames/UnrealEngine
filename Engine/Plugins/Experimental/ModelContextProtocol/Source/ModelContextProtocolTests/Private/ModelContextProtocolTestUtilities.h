// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelContextProtocol.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "HttpFwd.h"
#include "Interfaces/IHttpResponse.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

namespace UE::ModelContextProtocol::Tests
{
	inline FString GetTestBaseUrl()
	{
		return FString::Printf(TEXT("http://127.0.0.1:%d%s"), UE::ModelContextProtocol::DefaultServerPort, UE::ModelContextProtocol::DefaultServerUrlPath);
	}

	inline FString JsonObjectToString(const TSharedRef<FJsonObject>& JsonObject)
	{
		FString OutputString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
		FJsonSerializer::Serialize(JsonObject, Writer);
		return OutputString;
	}

	inline TSharedRef<FJsonObject> MakeJsonRpcRequest(const FString& Method, const TSharedPtr<FJsonValue>& RequestId = MakeShared<FJsonValueNumber>(1), const TSharedPtr<FJsonObject>& Params = nullptr)
	{
		TSharedRef<FJsonObject> Request = MakeShared<FJsonObject>();
		Request->SetStringField(TEXT("jsonrpc"), UE::ModelContextProtocol::JsonRpcVersion);
		Request->SetStringField(TEXT("method"), Method);
		if (RequestId.IsValid())
		{
			Request->SetField(TEXT("id"), RequestId);
		}
		if (Params.IsValid())
		{
			Request->SetObjectField(TEXT("params"), Params);
		}
		return Request;
	}

	inline TSharedRef<FJsonObject> MakeJsonRpcNotification(const FString& Method, const TSharedPtr<FJsonObject>& Params = nullptr)
	{
		TSharedRef<FJsonObject> Request = MakeShared<FJsonObject>();
		Request->SetStringField(TEXT("jsonrpc"), UE::ModelContextProtocol::JsonRpcVersion);
		Request->SetStringField(TEXT("method"), Method);
		if (Params.IsValid())
		{
			Request->SetObjectField(TEXT("params"), Params);
		}
		return Request;
	}

	/** Parse the first SSE "data:" line from a text/event-stream response into a JSON object. */
	inline TSharedPtr<FJsonObject> ParseSseJsonResponse(FHttpResponsePtr Response)
	{
		if (!Response.IsValid())
		{
			return nullptr;
		}
		FString Content = Response->GetContentAsString();
		// SSE format: "data: {json}\n\n" — extract only the first data line's payload
		int32 DataIndex = Content.Find(TEXT("data: "));
		if (DataIndex == INDEX_NONE)
		{
			return nullptr;
		}
		FString JsonStr = Content.Mid(DataIndex + 6);
		// Truncate at the first newline to isolate the single data line
		int32 NewlineIndex;
		if (JsonStr.FindChar(TEXT('\n'), NewlineIndex))
		{
			JsonStr.LeftInline(NewlineIndex);
		}
		JsonStr.TrimEndInline();
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonStr);
		if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
		{
			UE_LOGF(LogModelContextProtocol, Warning, "ParseSseJsonResponse: Failed to parse JSON from SSE data line: %ls", *JsonStr);
			return nullptr;
		}
		return JsonObject;
	}
}
