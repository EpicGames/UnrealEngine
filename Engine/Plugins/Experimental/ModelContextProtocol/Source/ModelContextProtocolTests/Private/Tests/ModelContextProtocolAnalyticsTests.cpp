// Copyright Epic Games, Inc. All Rights Reserved.

#include "IModelContextProtocolModule.h"
#include "Mocks/MockAnalyticsProviderET.h"
#include "ModelContextProtocolAnalytics.h"

#include "Dom/JsonObject.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolAnalyticsTests, "AI.ModelContextProtocol.Analytics", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	IModelContextProtocolModule* Module = nullptr;
	TSharedPtr<IAnalyticsProviderET> PreviousProvider;
	FString PreviousNamespace;
	TSharedPtr<FMockAnalyticsProviderET> MockProvider;
	bool bPreviousEnabled = true;

	IConsoleVariable* GetEnableAnalyticsCVar() const
	{
		return IConsoleManager::Get().FindConsoleVariable(TEXT("ModelContextProtocol.EnableAnalytics"));
	}
END_DEFINE_SPEC(FModelContextProtocolAnalyticsTests)

void FModelContextProtocolAnalyticsTests::Define()
{
	BeforeEach([this]()
	{
		Module = IModelContextProtocolModule::Get();
		MockProvider = MakeShared<FMockAnalyticsProviderET>();

		if (Module)
		{
			PreviousProvider = Module->GetAnalyticsProvider();
			PreviousNamespace = Module->GetAnalyticsEventNamespace();
			Module->SetAnalyticsProvider(MockProvider);
		}

		if (IConsoleVariable* CVar = GetEnableAnalyticsCVar())
		{
			bPreviousEnabled = CVar->GetBool();
			CVar->Set(true, ECVF_SetByCode);
		}
	});

	AfterEach([this]()
	{
		if (Module)
		{
			Module->SetAnalyticsProvider(PreviousProvider);
			Module->SetAnalyticsEventNamespace(PreviousNamespace);
		}
		if (IConsoleVariable* CVar = GetEnableAnalyticsCVar())
		{
			CVar->Set(bPreviousEnabled, ECVF_SetByCode);
		}
		MockProvider.Reset();
		PreviousProvider.Reset();
	});

	Describe("RecordToolCallEvent", [this]()
	{
		It("should record a ToolCall event with the expected attributes", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }

			UE::ModelContextProtocol::Analytics::RecordToolCallEvent(
				TEXT("test-session-id"),
				TEXT("toolset_registry.toolsets.core.scene.SceneTools.add_to_scene_from_asset"),
				0.125,
				true);

			if (!TestEqual("Should record exactly one event", MockProvider->RecordedEvents.Num(), 1)) { return; }

			const FMockAnalyticsProviderET::FRecordedEvent& Event = MockProvider->RecordedEvents[0];
			TestEqual("Event name should be namespaced", Event.EventName, TEXT("ModelContextProtocol.ToolCall"));

			const FAnalyticsEventAttribute* SessionId = MockProvider->FindAttribute(Event, TEXT("SessionId"));
			if (TestNotNull("SessionId attribute should exist", SessionId))
			{
				TestEqual("SessionId should match", SessionId->GetValue(), TEXT("test-session-id"));
			}

			const FAnalyticsEventAttribute* ToolsetNameHash = MockProvider->FindAttribute(Event, TEXT("ToolsetNameHash"));
			if (TestNotNull("ToolsetNameHash attribute should exist", ToolsetNameHash))
			{
				const FString ExpectedToolsetHash = UE::ModelContextProtocol::Analytics::HashToolIdentifier(TEXT("toolset_registry.toolsets.core.scene.SceneTools"));
				TestEqual("ToolsetNameHash should match Blake3 hash of dotted prefix", ToolsetNameHash->GetValue(), ExpectedToolsetHash);
			}

			const FAnalyticsEventAttribute* ToolNameHash = MockProvider->FindAttribute(Event, TEXT("ToolNameHash"));
			if (TestNotNull("ToolNameHash attribute should exist", ToolNameHash))
			{
				const FString ExpectedToolHash = UE::ModelContextProtocol::Analytics::HashToolIdentifier(TEXT("toolset_registry.toolsets.core.scene.SceneTools.add_to_scene_from_asset"));
				TestEqual("ToolNameHash should match Blake3 hash of full name", ToolNameHash->GetValue(), ExpectedToolHash);
			}

			TestNull("ToolsetName plaintext should not be emitted", MockProvider->FindAttribute(Event, TEXT("ToolsetName")));
			TestNull("ToolName plaintext should not be emitted", MockProvider->FindAttribute(Event, TEXT("ToolName")));

			TestNotNull("Duration attribute should exist", MockProvider->FindAttribute(Event, TEXT("Duration")));

			const FAnalyticsEventAttribute* ResultStatus = MockProvider->FindAttribute(Event, TEXT("ResultStatus"));
			if (TestNotNull("ResultStatus attribute should exist", ResultStatus))
			{
				TestEqual("ResultStatus should be Success", ResultStatus->GetValue(), TEXT("Success"));
			}
		});

		It("should emit ResultStatus=Error for failed tool calls", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }

			UE::ModelContextProtocol::Analytics::RecordToolCallEvent(
				TEXT("session"), TEXT("ns.tool"), 0.0, /*bSuccess*/false);

			if (!TestEqual("Should record one event", MockProvider->RecordedEvents.Num(), 1)) { return; }

			const FAnalyticsEventAttribute* ResultStatus = MockProvider->FindAttribute(MockProvider->RecordedEvents[0], TEXT("ResultStatus"));
			if (TestNotNull("ResultStatus attribute should exist", ResultStatus))
			{
				TestEqual("ResultStatus should be Error", ResultStatus->GetValue(), TEXT("Error"));
			}
		});

		It("should hash an empty toolset prefix for tool names without a dot", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }

			UE::ModelContextProtocol::Analytics::RecordToolCallEvent(
				TEXT("session"), TEXT("bare_tool"), 0.0, true);

			if (!TestEqual("Should record one event", MockProvider->RecordedEvents.Num(), 1)) { return; }

			const FAnalyticsEventAttribute* ToolsetNameHash = MockProvider->FindAttribute(MockProvider->RecordedEvents[0], TEXT("ToolsetNameHash"));
			if (TestNotNull("ToolsetNameHash attribute should exist", ToolsetNameHash))
			{
				const FString ExpectedEmptyHash = UE::ModelContextProtocol::Analytics::HashToolIdentifier(FString());
				TestEqual("ToolsetNameHash for undotted tool should be the hash of the empty string", ToolsetNameHash->GetValue(), ExpectedEmptyHash);
			}
		});
	});

	Describe("RecordSessionStartEvent", [this]()
	{
		It("should record a SessionStart event with SessionId and ProtocolVersion", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }

			UE::ModelContextProtocol::Analytics::RecordSessionStartEvent(TEXT("session-abc"), TEXT("2025-11-25"));

			if (!TestEqual("Should record one event", MockProvider->RecordedEvents.Num(), 1)) { return; }

			const FMockAnalyticsProviderET::FRecordedEvent& Event = MockProvider->RecordedEvents[0];
			TestEqual("Event name should be namespaced", Event.EventName, TEXT("ModelContextProtocol.SessionStart"));

			const FAnalyticsEventAttribute* SessionId = MockProvider->FindAttribute(Event, TEXT("SessionId"));
			if (TestNotNull("SessionId attribute should exist", SessionId))
			{
				TestEqual("SessionId should match", SessionId->GetValue(), TEXT("session-abc"));
			}

			const FAnalyticsEventAttribute* ProtocolVersion = MockProvider->FindAttribute(Event, TEXT("ProtocolVersion"));
			if (TestNotNull("ProtocolVersion attribute should exist", ProtocolVersion))
			{
				TestEqual("ProtocolVersion should match", ProtocolVersion->GetValue(), TEXT("2025-11-25"));
			}
		});
	});

	Describe("RecordSessionEndEvent", [this]()
	{
		It("should record a SessionEnd event with SessionId", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }

			UE::ModelContextProtocol::Analytics::RecordSessionEndEvent(TEXT("session-xyz"));

			if (!TestEqual("Should record one event", MockProvider->RecordedEvents.Num(), 1)) { return; }

			const FMockAnalyticsProviderET::FRecordedEvent& Event = MockProvider->RecordedEvents[0];
			TestEqual("Event name should be namespaced", Event.EventName, TEXT("ModelContextProtocol.SessionEnd"));

			const FAnalyticsEventAttribute* SessionId = MockProvider->FindAttribute(Event, TEXT("SessionId"));
			if (TestNotNull("SessionId attribute should exist", SessionId))
			{
				TestEqual("SessionId should match", SessionId->GetValue(), TEXT("session-xyz"));
			}
		});
	});

	Describe("Namespace", [this]()
	{
		It("should use the namespace set via SetAnalyticsEventNamespace", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }

			Module->SetAnalyticsEventNamespace(TEXT("CustomNS"));

			UE::ModelContextProtocol::Analytics::RecordSessionEndEvent(TEXT("s"));

			if (!TestEqual("Should record one event", MockProvider->RecordedEvents.Num(), 1)) { return; }
			TestEqual("Event name should use the custom namespace", MockProvider->RecordedEvents[0].EventName, TEXT("CustomNS.SessionEnd"));
		});

		It("should emit unprefixed event names when namespace is empty", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }

			Module->SetAnalyticsEventNamespace(FString());

			UE::ModelContextProtocol::Analytics::RecordSessionEndEvent(TEXT("s"));

			if (!TestEqual("Should record one event", MockProvider->RecordedEvents.Num(), 1)) { return; }
			TestEqual("Event name should have no prefix", MockProvider->RecordedEvents[0].EventName, TEXT("SessionEnd"));
		});
	});

	Describe("HashToolIdentifier", [this]()
	{
		It("should return a 64-character lowercase hex string", [this]()
		{
			const FString Hash = UE::ModelContextProtocol::Analytics::HashToolIdentifier(TEXT("toolset_registry.toolsets.core.scene.SceneTools.add_to_scene_from_asset"));
			TestEqual("Blake3 hex output should be 64 characters", Hash.Len(), 64);
			for (const TCHAR C : Hash)
			{
				const bool bIsLowercaseHex = (C >= TEXT('0') && C <= TEXT('9')) || (C >= TEXT('a') && C <= TEXT('f'));
				if (!bIsLowercaseHex)
				{
					AddError(FString::Printf(TEXT("Unexpected character '%c' in hash output"), C));
					break;
				}
			}
		});

		It("should be deterministic for the same input", [this]()
		{
			const FString A = UE::ModelContextProtocol::Analytics::HashToolIdentifier(TEXT("SameInput"));
			const FString B = UE::ModelContextProtocol::Analytics::HashToolIdentifier(TEXT("SameInput"));
			TestEqual("Two hashes of the same input should match", A, B);
		});

		It("should differ between distinct inputs", [this]()
		{
			const FString A = UE::ModelContextProtocol::Analytics::HashToolIdentifier(TEXT("Toolset.ToolA"));
			const FString B = UE::ModelContextProtocol::Analytics::HashToolIdentifier(TEXT("Toolset.ToolB"));
			TestNotEqual("Distinct inputs should produce distinct hashes", A, B);
		});

		It("should produce a stable hash for the empty string", [this]()
		{
			const FString Hash = UE::ModelContextProtocol::Analytics::HashToolIdentifier(FString());
			TestEqual("Empty-string Blake3 hash is deterministic and 64 chars",
				Hash.Len(), 64);
		});
	});

	Describe("IsToolResultSuccess", [this]()
	{
		It("should return false for a null JsonObject", [this]()
		{
			TestFalse("Null result is treated as error", UE::ModelContextProtocol::Analytics::IsToolResultSuccess(nullptr));
		});

		It("should return true when the isError field is absent", [this]()
		{
			TSharedPtr<FJsonObject> ResultJson = MakeShared<FJsonObject>();
			TestTrue("Missing isError defaults to success", UE::ModelContextProtocol::Analytics::IsToolResultSuccess(ResultJson));
		});

		It("should return true when isError is false", [this]()
		{
			TSharedPtr<FJsonObject> ResultJson = MakeShared<FJsonObject>();
			ResultJson->SetBoolField(TEXT("isError"), false);
			TestTrue("isError=false is success", UE::ModelContextProtocol::Analytics::IsToolResultSuccess(ResultJson));
		});

		It("should return false when isError is true", [this]()
		{
			TSharedPtr<FJsonObject> ResultJson = MakeShared<FJsonObject>();
			ResultJson->SetBoolField(TEXT("isError"), true);
			TestFalse("isError=true is error", UE::ModelContextProtocol::Analytics::IsToolResultSuccess(ResultJson));
		});

		It("should return false when isError is a non-bool (malformed)", [this]()
		{
			AddExpectedError(TEXT("non-bool 'isError'"), EAutomationExpectedErrorFlags::Contains);
			TSharedPtr<FJsonObject> ResultJson = MakeShared<FJsonObject>();
			ResultJson->SetStringField(TEXT("isError"), TEXT("yes"));
			TestFalse("Malformed isError is treated as error", UE::ModelContextProtocol::Analytics::IsToolResultSuccess(ResultJson));
		});
	});

	Describe("Gating", [this]()
	{
		It("should not record events when ModelContextProtocol.EnableAnalytics is false", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }

			IConsoleVariable* CVar = GetEnableAnalyticsCVar();
			if (!TestNotNull("EnableAnalytics CVar should exist", CVar)) { return; }

			CVar->Set(false, ECVF_SetByCode);
			UE::ModelContextProtocol::Analytics::RecordToolCallEvent(TEXT("s"), TEXT("t"), 0.0, true);
			UE::ModelContextProtocol::Analytics::RecordSessionStartEvent(TEXT("s"), TEXT("v"));
			UE::ModelContextProtocol::Analytics::RecordSessionEndEvent(TEXT("s"));

			TestEqual("No events should be recorded while disabled", MockProvider->RecordedEvents.Num(), 0);
		});

		It("should not crash when no analytics provider is set", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }

			Module->SetAnalyticsProvider(nullptr);
			UE::ModelContextProtocol::Analytics::RecordToolCallEvent(TEXT("s"), TEXT("t"), 0.0, true);
			TestEqual("No events should be recorded without a provider", MockProvider->RecordedEvents.Num(), 0);
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
