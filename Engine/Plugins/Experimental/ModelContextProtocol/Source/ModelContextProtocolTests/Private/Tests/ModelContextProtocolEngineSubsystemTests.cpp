// Copyright Epic Games, Inc. All Rights Reserved.

#include "IModelContextProtocolModule.h"
#include "IModelContextProtocolTool.h"
#include "IModelContextProtocolResourceProvider.h"
#include "ModelContextProtocol.h"
#include "ModelContextProtocolResources.h"
#include "ModelContextProtocolTestUtilities.h"
#include "Mocks/MockModelContextProtocolTool.h"
#include "Mocks/MockModelContextProtocolResourceProvider.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "Misc/AutomationTest.h"
#include "Misc/Base64.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolServerTests, "AI.ModelContextProtocol.Server", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	FString SessionId;
	TSharedPtr<FMockModelContextProtocolTool> MockTool;
	TArray<TSharedPtr<FMockModelContextProtocolTool>> AdditionalMockTools;
	TSharedPtr<FMockModelContextProtocolResourceProvider> MockResourceProvider;

	void SendJsonRpcRequest(const TSharedRef<FJsonObject>& JsonRpcBody, const TFunction<void(FHttpResponsePtr, bool)>& OnComplete, const FString& InSessionId = FString())
	{
		FHttpModule& HttpModule = FHttpModule::Get();
		TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
		HttpRequest->SetURL(UE::ModelContextProtocol::Tests::GetTestBaseUrl());
		HttpRequest->SetVerb(TEXT("POST"));
		HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
		if (!InSessionId.IsEmpty())
		{
			HttpRequest->SetHeader(TEXT("Mcp-Session-Id"), InSessionId);
		}
		HttpRequest->SetContentAsString(UE::ModelContextProtocol::Tests::JsonObjectToString(JsonRpcBody));
		HttpRequest->OnProcessRequestComplete().BindLambda([OnComplete](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnectedSuccessfully)
		{
			OnComplete(Response, bConnectedSuccessfully);
		});
		HttpRequest->ProcessRequest();
	}

	TSharedPtr<FJsonObject> ParseJsonResponse(FHttpResponsePtr Response)
	{
		if (!Response.IsValid())
		{
			return nullptr;
		}
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Response->GetContentAsString());
		FJsonSerializer::Deserialize(Reader, JsonObject);
		return JsonObject;
	}

	void InitializeSession(const FDoneDelegate& Done)
	{
		using namespace UE::ModelContextProtocol::Tests;

		TSharedRef<FJsonObject> InitParams = MakeShared<FJsonObject>();
		InitParams->SetStringField(TEXT("protocolVersion"), UE::ModelContextProtocol::ProtocolVersion);
		TSharedRef<FJsonObject> ClientInfo = MakeShared<FJsonObject>();
		ClientInfo->SetStringField(TEXT("name"), TEXT("TestClient"));
		ClientInfo->SetStringField(TEXT("version"), TEXT("1.0"));
		InitParams->SetObjectField(TEXT("clientInfo"), ClientInfo);
		InitParams->SetObjectField(TEXT("capabilities"), MakeShared<FJsonObject>());

		TSharedRef<FJsonObject> InitRequest = MakeJsonRpcRequest(TEXT("initialize"), MakeShared<FJsonValueNumber>(1), InitParams);
		SendJsonRpcRequest(InitRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
		{
			if (Response.IsValid())
			{
				SessionId = Response->GetHeader(TEXT("Mcp-Session-Id"));

				// Send initialized notification
				TSharedRef<FJsonObject> InitializedNotification = UE::ModelContextProtocol::Tests::MakeJsonRpcNotification(TEXT("notifications/initialized"));
				SendJsonRpcRequest(InitializedNotification, [Done](FHttpResponsePtr, bool)
				{
					Done.Execute();
				}, SessionId);
			}
			else
			{
				Done.Execute();
			}
		});
	}

	void CleanupSession(const FDoneDelegate& Done)
	{
		if (SessionId.IsEmpty())
		{
			Done.Execute();
			return;
		}

		FHttpModule& HttpModule = FHttpModule::Get();
		TSharedRef<IHttpRequest> DeleteRequest = HttpModule.CreateRequest();
		DeleteRequest->SetURL(UE::ModelContextProtocol::Tests::GetTestBaseUrl());
		DeleteRequest->SetVerb(TEXT("DELETE"));
		DeleteRequest->SetHeader(TEXT("Mcp-Session-Id"), SessionId);
		DeleteRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr, bool)
		{
			SessionId.Reset();
			Done.Execute();
		});
		DeleteRequest->ProcessRequest();
	}
END_DEFINE_SPEC(FModelContextProtocolServerTests)

void FModelContextProtocolServerTests::Define()
{
	using namespace UE::ModelContextProtocol::Tests;

	BeforeEach([this]()
	{
		MockTool = MakeShared<FMockModelContextProtocolTool>();
		MockResourceProvider = MakeShared<FMockModelContextProtocolResourceProvider>();
		SessionId.Reset();
	});

	AfterEach([this]()
	{
		if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
		{
			if (MockTool.IsValid())
			{
				Module->RemoveTool(MockTool.ToSharedRef());
			}
			for (const TSharedPtr<FMockModelContextProtocolTool>& Tool : AdditionalMockTools)
			{
				if (Tool.IsValid())
				{
					Module->RemoveTool(Tool.ToSharedRef());
				}
			}
			if (MockResourceProvider.IsValid())
			{
				Module->RemoveResourceProvider(MockResourceProvider.ToSharedRef());
			}
		}
		MockTool.Reset();
		AdditionalMockTools.Reset();
		MockResourceProvider.Reset();
	});

	Describe("HTTP Method Handling", [this]()
	{
		LatentIt("should reject GET requests with 405 Method Not Allowed", [this](const FDoneDelegate& Done)
		{
			FHttpModule& HttpModule = FHttpModule::Get();
			TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
			HttpRequest->SetURL(GetTestBaseUrl());
			HttpRequest->SetVerb(TEXT("GET"));
			HttpRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 405 for GET", Response->GetResponseCode(), 405);
				}
				Done.Execute();
			});
			HttpRequest->ProcessRequest();
		});
	});

	Describe("JSON-RPC Validation", [this]()
	{
		LatentIt("should reject invalid JSON with ParseError", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Invalid JSON body"), EAutomationExpectedErrorFlags::Contains);
			FHttpModule& HttpModule = FHttpModule::Get();
			TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
			HttpRequest->SetURL(GetTestBaseUrl());
			HttpRequest->SetVerb(TEXT("POST"));
			HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
			HttpRequest->SetContentAsString(TEXT("{invalid json}"));
			HttpRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
					if (TestTrue("Should parse response", JsonResponse.IsValid()))
					{
						const TSharedPtr<FJsonObject>* ErrorObject;
						if (TestTrue("Should have error field", JsonResponse->TryGetObjectField(TEXT("error"), ErrorObject)))
						{
							int32 ErrorCode = 0;
							if (TestTrue("Should have error code", (*ErrorObject)->TryGetNumberField(TEXT("code"), ErrorCode)))
							{
								TestEqual("Error code should be ParseError (-32700)", ErrorCode, -32700);
							}
						}
					}
				}
				Done.Execute();
			});
			HttpRequest->ProcessRequest();
		});

		LatentIt("should reject missing jsonrpc version", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Unexpected jsonrpc version"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> BadRequest = MakeShared<FJsonObject>();
			BadRequest->SetStringField(TEXT("method"), TEXT("ping"));
			BadRequest->SetField(TEXT("id"), MakeShared<FJsonValueNumber>(1));

			SendJsonRpcRequest(BadRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
					if (TestTrue("Should parse response", JsonResponse.IsValid()))
					{
						const TSharedPtr<FJsonObject>* ErrorObject;
						if (TestTrue("Should have error field", JsonResponse->TryGetObjectField(TEXT("error"), ErrorObject)))
						{
							int32 ErrorCode = 0;
							if (TestTrue("Should have error code", (*ErrorObject)->TryGetNumberField(TEXT("code"), ErrorCode)))
							{
								TestEqual("Error code should be InvalidRequest (-32600)", ErrorCode, -32600);
							}
						}
					}
				}
				Done.Execute();
			});
		});

		LatentIt("should reject unknown method with MethodNotFound", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("unknown method"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("nonexistent/method"));
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
					if (TestTrue("Should parse response", JsonResponse.IsValid()))
					{
						const TSharedPtr<FJsonObject>* ErrorObject;
						if (TestTrue("Should have error field", JsonResponse->TryGetObjectField(TEXT("error"), ErrorObject)))
						{
							int32 ErrorCode = 0;
							if (TestTrue("Should have error code", (*ErrorObject)->TryGetNumberField(TEXT("code"), ErrorCode)))
							{
								TestEqual("Error code should be MethodNotFound (-32601)", ErrorCode, -32601);
							}
						}
					}
				}
				Done.Execute();
			});
		});
	});

	Describe("Protocol Operations", [this]()
	{
		LatentIt("should respond to ping with empty result", [this](const FDoneDelegate& Done)
		{
			TSharedRef<FJsonObject> PingRequest = MakeJsonRpcRequest(TEXT("ping"));
			SendJsonRpcRequest(PingRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
					if (TestTrue("Should parse response", JsonResponse.IsValid()))
					{
						TestTrue("Should have result field", JsonResponse->HasField(TEXT("result")));
					}
				}
				Done.Execute();
			});
		});

		LatentIt("should create session on initialize and return capabilities", [this](const FDoneDelegate& Done)
		{
			TSharedRef<FJsonObject> InitParams = MakeShared<FJsonObject>();
			InitParams->SetStringField(TEXT("protocolVersion"), UE::ModelContextProtocol::ProtocolVersion);
			TSharedRef<FJsonObject> ClientInfo = MakeShared<FJsonObject>();
			ClientInfo->SetStringField(TEXT("name"), TEXT("TestClient"));
			ClientInfo->SetStringField(TEXT("version"), TEXT("1.0"));
			InitParams->SetObjectField(TEXT("clientInfo"), ClientInfo);
			InitParams->SetObjectField(TEXT("capabilities"), MakeShared<FJsonObject>());

			TSharedRef<FJsonObject> InitRequest = MakeJsonRpcRequest(TEXT("initialize"), MakeShared<FJsonValueNumber>(1), InitParams);
			SendJsonRpcRequest(InitRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					FString NewSessionId = Response->GetHeader(TEXT("Mcp-Session-Id"));
					TestFalse("Should have Mcp-Session-Id header", NewSessionId.IsEmpty());

					TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
					if (TestTrue("Should parse response", JsonResponse.IsValid()))
					{
						const TSharedPtr<FJsonObject>* ResultObject;
						if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
						{
							TestTrue("Should have capabilities", (*ResultObject)->HasField(TEXT("capabilities")));
							TestTrue("Should have serverInfo", (*ResultObject)->HasField(TEXT("serverInfo")));
							TestTrue("Should have protocolVersion", (*ResultObject)->HasField(TEXT("protocolVersion")));

							// Verify tools capability advertises listChanged
							const TSharedPtr<FJsonObject>* CapabilitiesObject;
							if (TestTrue("Should have capabilities object", (*ResultObject)->TryGetObjectField(TEXT("capabilities"), CapabilitiesObject)))
							{
								const TSharedPtr<FJsonObject>* ToolsObject;
								if (TestTrue("Should have tools capability", (*CapabilitiesObject)->TryGetObjectField(TEXT("tools"), ToolsObject)))
								{
									bool bListChanged = false;
									TestTrue("Should have tools.listChanged", (*ToolsObject)->TryGetBoolField(TEXT("listChanged"), bListChanged));
									TestTrue("tools.listChanged should be true", bListChanged);
								}

								TestTrue("Should have resources capability", (*CapabilitiesObject)->HasField(TEXT("resources")));
							}
						}
					}

					// Clean up the session
					SessionId = NewSessionId;
					CleanupSession(Done);
				}
				else
				{
					Done.Execute();
				}
			});
		});

		LatentIt("should echo supported protocol version back to client", [this](const FDoneDelegate& Done)
		{
			TSharedRef<FJsonObject> InitParams = MakeShared<FJsonObject>();
			InitParams->SetStringField(TEXT("protocolVersion"), TEXT("2024-11-05"));
			TSharedRef<FJsonObject> ClientInfo = MakeShared<FJsonObject>();
			ClientInfo->SetStringField(TEXT("name"), TEXT("TestClient"));
			ClientInfo->SetStringField(TEXT("version"), TEXT("1.0"));
			InitParams->SetObjectField(TEXT("clientInfo"), ClientInfo);
			InitParams->SetObjectField(TEXT("capabilities"), MakeShared<FJsonObject>());

			TSharedRef<FJsonObject> InitRequest = MakeJsonRpcRequest(TEXT("initialize"), MakeShared<FJsonValueNumber>(1), InitParams);
			SendJsonRpcRequest(InitRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (!TestTrue("Should connect", bConnected) || !TestTrue("Should have response", Response.IsValid()))
				{
					Done.Execute();
					return;
				}

				TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
				if (!TestTrue("Should parse response", JsonResponse.IsValid()))
				{
					Done.Execute();
					return;
				}

				const TSharedPtr<FJsonObject>* ResultObject;
				if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
				{
					FString ResponseVersion;
					if (TestTrue("Should have protocolVersion", (*ResultObject)->TryGetStringField(TEXT("protocolVersion"), ResponseVersion)))
					{
						TestEqual("Server should echo the client's supported version", ResponseVersion, TEXT("2024-11-05"));
					}
				}

				SessionId = Response->GetHeader(TEXT("Mcp-Session-Id"));
				CleanupSession(Done);
			});
		});

		LatentIt("should return server's latest version for unsupported client version", [this](const FDoneDelegate& Done)
		{
			TSharedRef<FJsonObject> InitParams = MakeShared<FJsonObject>();
			InitParams->SetStringField(TEXT("protocolVersion"), TEXT("1999-01-01"));
			TSharedRef<FJsonObject> ClientInfo = MakeShared<FJsonObject>();
			ClientInfo->SetStringField(TEXT("name"), TEXT("TestClient"));
			ClientInfo->SetStringField(TEXT("version"), TEXT("1.0"));
			InitParams->SetObjectField(TEXT("clientInfo"), ClientInfo);
			InitParams->SetObjectField(TEXT("capabilities"), MakeShared<FJsonObject>());

			TSharedRef<FJsonObject> InitRequest = MakeJsonRpcRequest(TEXT("initialize"), MakeShared<FJsonValueNumber>(1), InitParams);
			SendJsonRpcRequest(InitRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (!TestTrue("Should connect", bConnected) || !TestTrue("Should have response", Response.IsValid()))
				{
					Done.Execute();
					return;
				}

				TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
				if (!TestTrue("Should parse response", JsonResponse.IsValid()))
				{
					Done.Execute();
					return;
				}

				const TSharedPtr<FJsonObject>* ResultObject;
				if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
				{
					FString ResponseVersion;
					if (TestTrue("Should have protocolVersion", (*ResultObject)->TryGetStringField(TEXT("protocolVersion"), ResponseVersion)))
					{
						TestEqual("Server should return its latest version", ResponseVersion, UE::ModelContextProtocol::ProtocolVersion);
					}
				}

				SessionId = Response->GetHeader(TEXT("Mcp-Session-Id"));
				CleanupSession(Done);
			});
		});

		LatentIt("should accept notifications/initialized with 202", [this](const FDoneDelegate& Done)
		{
			// First initialize
			TSharedRef<FJsonObject> InitParams = MakeShared<FJsonObject>();
			InitParams->SetStringField(TEXT("protocolVersion"), UE::ModelContextProtocol::ProtocolVersion);
			TSharedRef<FJsonObject> ClientInfo = MakeShared<FJsonObject>();
			ClientInfo->SetStringField(TEXT("name"), TEXT("TestClient"));
			ClientInfo->SetStringField(TEXT("version"), TEXT("1.0"));
			InitParams->SetObjectField(TEXT("clientInfo"), ClientInfo);
			InitParams->SetObjectField(TEXT("capabilities"), MakeShared<FJsonObject>());

			TSharedRef<FJsonObject> InitRequest = MakeJsonRpcRequest(TEXT("initialize"), MakeShared<FJsonValueNumber>(1), InitParams);
			SendJsonRpcRequest(InitRequest, [this, Done](FHttpResponsePtr InitResponse, bool bConnected)
			{
				if (!TestTrue("Should connect", bConnected) || !TestTrue("Should have response", InitResponse.IsValid()))
				{
					Done.Execute();
					return;
				}

				SessionId = InitResponse->GetHeader(TEXT("Mcp-Session-Id"));
				TSharedRef<FJsonObject> InitializedNotification = MakeJsonRpcNotification(TEXT("notifications/initialized"));
				SendJsonRpcRequest(InitializedNotification, [this, Done](FHttpResponsePtr Response, bool bNotifConnected)
				{
					if (TestTrue("Should connect", bNotifConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TestEqual("Should return 202", Response->GetResponseCode(), 202);
					}
					CleanupSession(Done);
				}, SessionId);
			});
		});

		LatentIt("should return registered tools via tools/list", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				MockTool->Name = TEXT("test_greet_tool");
				MockTool->Description = TEXT("A test tool that greets a user by name");
				MockTool->InputSchema = FMockModelContextProtocolTool::MakeTestInputSchema(
					{{TEXT("userName"), TEXT("string")}},
					{TEXT("userName")});
				MockTool->OutputSchema = FMockModelContextProtocolTool::MakeTestInputSchema(
					{{TEXT("greeting"), TEXT("string")}});
				if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
				{
					Module->AddTool(MockTool.ToSharedRef());
				}

				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(2));
				SendJsonRpcRequest(ListRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ToolsArray;
								if (TestTrue("Should have tools array", (*ResultObject)->TryGetArrayField(TEXT("tools"), ToolsArray)))
								{
									bool bFoundTool = false;
									for (const TSharedPtr<FJsonValue>& ToolValue : *ToolsArray)
									{
										const TSharedPtr<FJsonObject>* ToolObject;
										if (ToolValue->TryGetObject(ToolObject))
										{
											FString ToolName;
											if ((*ToolObject)->TryGetStringField(TEXT("name"), ToolName) && ToolName == TEXT("test_greet_tool"))
											{
												bFoundTool = true;
												// Verify tool metadata is included in listing
												FString ToolDescription;
												(*ToolObject)->TryGetStringField(TEXT("description"), ToolDescription);
												TestEqual("Should include tool description", ToolDescription, TEXT("A test tool that greets a user by name"));
												const TSharedPtr<FJsonObject>* InputSchemaObject;
												TestTrue("Should include inputSchema", (*ToolObject)->TryGetObjectField(TEXT("inputSchema"), InputSchemaObject));
												const TSharedPtr<FJsonObject>* OutputSchemaObject;
												TestTrue("Should include outputSchema", (*ToolObject)->TryGetObjectField(TEXT("outputSchema"), OutputSchemaObject));
											}
										}
									}
									TestTrue("Should find the registered test tool by name", bFoundTool);
								}
							}
						}
					}
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should execute sync tool via tools/call", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				MockTool->Name = TEXT("test_echo_tool");
				MockTool->Description = TEXT("A test tool that echoes back a greeting for a given user");
				MockTool->InputSchema = FMockModelContextProtocolTool::MakeTestInputSchema(
					{{TEXT("userName"), TEXT("string")}},
					{TEXT("userName")});
				MockTool->OutputSchema = FMockModelContextProtocolTool::MakeTestInputSchema(
					{{TEXT("greeting"), TEXT("string")}});
				MockTool->RunFunction = [](const TSharedPtr<FJsonObject>& Params)
				{
					FString UserName;
					if (Params.IsValid())
					{
						Params->TryGetStringField(TEXT("userName"), UserName);
					}
					return UE::ModelContextProtocol::MakeTextResult(FString::Printf(TEXT("Hello, %s!"), *UserName));
				};
				if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
				{
					Module->AddTool(MockTool.ToSharedRef());
				}

				TSharedRef<FJsonObject> ToolArguments = MakeShared<FJsonObject>();
				ToolArguments->SetStringField(TEXT("userName"), TEXT("TestUser"));

				TSharedRef<FJsonObject> CallParams = MakeShared<FJsonObject>();
				CallParams->SetStringField(TEXT("name"), TEXT("test_echo_tool"));
				CallParams->SetObjectField(TEXT("arguments"), ToolArguments);

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(3), CallParams);

				// tools/call returns an SSE stream. The UE HTTP server uses raw TCP without
				// Content-Length or chunked encoding for stream responses, so the HTTP client
				// cannot detect response completion and waits for the default activity timeout
				// (~30s). Use a short activity timeout since the mock tool completes synchronously
				// and all SSE data arrives immediately. The request will "fail" due to timeout but
				// the response body retains all buffered data for validation.
				FHttpModule& HttpModule = FHttpModule::Get();
				TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
				HttpRequest->SetURL(GetTestBaseUrl());
				HttpRequest->SetVerb(TEXT("POST"));
				HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
				HttpRequest->SetHeader(TEXT("Mcp-Session-Id"), SessionId);
				HttpRequest->SetActivityTimeout(2.0f);
				HttpRequest->SetContentAsString(JsonObjectToString(CallRequest));

				HttpRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool /*bConnected*/)
				{
					if (TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseSseJsonResponse(Response);
						if (TestTrue("Should parse SSE response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ContentArray;
								if (TestTrue("Should have content", (*ResultObject)->TryGetArrayField(TEXT("content"), ContentArray))
									&& TestTrue("Content should have at least 1 element", ContentArray->Num() >= 1))
								{
									const TSharedPtr<FJsonObject>* ContentObject;
									if ((*ContentArray)[0]->TryGetObject(ContentObject))
									{
										FString Text;
										if ((*ContentObject)->TryGetStringField(TEXT("text"), Text))
										{
											TestEqual("Tool result text should contain greeting", Text, TEXT("Hello, TestUser!"));
										}
									}
								}
							}
						}
					}
					CleanupSession(Done);
				});
				HttpRequest->ProcessRequest();
			}));
		});

		LatentIt("should error for unknown tool in tools/call", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Unknown tool"), EAutomationExpectedErrorFlags::Contains);
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> CallParams = MakeShared<FJsonObject>();
				CallParams->SetStringField(TEXT("name"), TEXT("test_nonexistent_tool"));
				CallParams->SetObjectField(TEXT("arguments"), MakeShared<FJsonObject>());

				TSharedRef<FJsonObject> CallRequest = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(4), CallParams);
				SendJsonRpcRequest(CallRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							// In non-editor: JSON-RPC error response with "error" field
							// In editor with ToolsetRegistry: result with isError=true
							const TSharedPtr<FJsonObject>* ErrorObject;
							bool bHasJsonRpcError = JsonResponse->TryGetObjectField(TEXT("error"), ErrorObject);
							const TSharedPtr<FJsonObject>* ResultObject;
							bool bHasErrorResult = false;
							if (JsonResponse->TryGetObjectField(TEXT("result"), ResultObject))
							{
								bool bIsError = false;
								(*ResultObject)->TryGetBoolField(TEXT("isError"), bIsError);
								bHasErrorResult = bIsError;
							}
							TestTrue("Should indicate error for unknown tool", bHasJsonRpcError || bHasErrorResult);
						}
					}
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should return resources via resources/list", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				MockResourceProvider->AddTextResource(
					TEXT("file:///test_integration.txt"),
					TEXT("sample content"),
					TOptional<FString>(TEXT("test_integration")),
					TOptional<FString>(TEXT("Test Integration Resource")),
					TOptional<FString>(TEXT("A test resource used for integration testing")),
					TOptional<FString>(TEXT("text/plain")));
				if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
				{
					Module->AddResourceProvider(MockResourceProvider.ToSharedRef());
				}

				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("resources/list"), MakeShared<FJsonValueNumber>(5));
				SendJsonRpcRequest(ListRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ResourcesArray;
								if (TestTrue("Should have resources array", (*ResultObject)->TryGetArrayField(TEXT("resources"), ResourcesArray)))
								{
									bool bFoundResource = false;
									for (const TSharedPtr<FJsonValue>& ResourceValue : *ResourcesArray)
									{
										const TSharedPtr<FJsonObject>* ResourceObject;
										if (ResourceValue->TryGetObject(ResourceObject))
										{
											FString Uri;
											if ((*ResourceObject)->TryGetStringField(TEXT("uri"), Uri) && Uri == TEXT("file:///test_integration.txt"))
											{
												bFoundResource = true;
												FString ResourceName;
												(*ResourceObject)->TryGetStringField(TEXT("name"), ResourceName);
												TestEqual("Should include resource name", ResourceName, TEXT("test_integration"));
											}
										}
									}
									TestTrue("Should find the registered test resource by URI", bFoundResource);
								}
							}
						}
					}
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should read a listed resource via resources/read", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				const FString TestUri = TEXT("file:///test_read_document.txt");
				MockResourceProvider->AddTextResource(
					TestUri,
					TEXT("The quick brown fox jumps over the lazy dog"),
					TOptional<FString>(TEXT("test_read_document")),
					TOptional<FString>(TEXT("Test Read Document")),
					TOptional<FString>(TEXT("A test text document for read integration testing")),
					TOptional<FString>(TEXT("text/plain")));
				if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
				{
					Module->AddResourceProvider(MockResourceProvider.ToSharedRef());
				}

				// First list resources to register the URI-to-provider mapping
				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("resources/list"), MakeShared<FJsonValueNumber>(6));
				SendJsonRpcRequest(ListRequest, [this, Done, TestUri](FHttpResponsePtr, bool)
				{
					TSharedRef<FJsonObject> ReadParams = MakeShared<FJsonObject>();
					ReadParams->SetStringField(TEXT("uri"), TestUri);
					TSharedRef<FJsonObject> ReadRequest = MakeJsonRpcRequest(TEXT("resources/read"), MakeShared<FJsonValueNumber>(7), ReadParams);
					SendJsonRpcRequest(ReadRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
					{
						if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
						{
							TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
							if (TestTrue("Should parse response", JsonResponse.IsValid()))
							{
								const TSharedPtr<FJsonObject>* ResultObject;
								if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
								{
									const TArray<TSharedPtr<FJsonValue>>* ContentsArray;
									if (TestTrue("Should have contents array", (*ResultObject)->TryGetArrayField(TEXT("contents"), ContentsArray))
										&& TestTrue("Contents should have at least 1 element", ContentsArray->Num() >= 1))
									{
										const TSharedPtr<FJsonObject>* ContentObject;
										if ((*ContentsArray)[0]->TryGetObject(ContentObject))
										{
											FString Text;
											if ((*ContentObject)->TryGetStringField(TEXT("text"), Text))
											{
												TestEqual("Resource text should match", Text, TEXT("The quick brown fox jumps over the lazy dog"));
											}
										}
									}
								}
							}
						}
						CleanupSession(Done);
					}, SessionId);
				}, SessionId);
			}));
		});

		LatentIt("should return ResourceNotFound (-32002) for unknown URI in resources/read", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Resource not found"), EAutomationExpectedErrorFlags::Contains);
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> ReadParams = MakeShared<FJsonObject>();
				ReadParams->SetStringField(TEXT("uri"), TEXT("file:///test_nonexistent_resource.txt"));
				TSharedRef<FJsonObject> ReadRequest = MakeJsonRpcRequest(TEXT("resources/read"), MakeShared<FJsonValueNumber>(8), ReadParams);
				SendJsonRpcRequest(ReadRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ErrorObject;
							if (TestTrue("Should have error for unknown URI", JsonResponse->TryGetObjectField(TEXT("error"), ErrorObject)))
							{
								int32 ErrorCode = 0;
								if (TestTrue("Should have error code", (*ErrorObject)->TryGetNumberField(TEXT("code"), ErrorCode)))
								{
									TestEqual("Error code should be ResourceNotFound (-32002)", ErrorCode, -32002);
								}
							}
						}
					}
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should remove session via DELETE", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				FHttpModule& HttpModule = FHttpModule::Get();
				TSharedRef<IHttpRequest> DeleteRequest = HttpModule.CreateRequest();
				DeleteRequest->SetURL(GetTestBaseUrl());
				DeleteRequest->SetVerb(TEXT("DELETE"));
				DeleteRequest->SetHeader(TEXT("Mcp-Session-Id"), SessionId);
				DeleteRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TestEqual("Should return 202 for successful delete", Response->GetResponseCode(), 202);
					}
					SessionId.Reset();
					Done.Execute();
				});
				DeleteRequest->ProcessRequest();
			}));
		});

		LatentIt("should accept notifications/cancelled", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> CancelParams = MakeShared<FJsonObject>();
				CancelParams->SetField(TEXT("requestId"), MakeShared<FJsonValueNumber>(999));
				TSharedRef<FJsonObject> CancelNotification = MakeJsonRpcNotification(TEXT("notifications/cancelled"), CancelParams);
				SendJsonRpcRequest(CancelNotification, [this, Done](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TestEqual("Should return 202 for cancellation notification", Response->GetResponseCode(), 202);
					}
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should error for DELETE with unknown session ID", [this](const FDoneDelegate& Done)
		{
			FHttpModule& HttpModule = FHttpModule::Get();
			TSharedRef<IHttpRequest> DeleteRequest = HttpModule.CreateRequest();
			DeleteRequest->SetURL(GetTestBaseUrl());
			DeleteRequest->SetVerb(TEXT("DELETE"));
			DeleteRequest->SetHeader(TEXT("Mcp-Session-Id"), TEXT("test-nonexistent-session-id"));
			DeleteRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestTrue("Should not return 200 for unknown session", Response->GetResponseCode() != 200);
				}
				Done.Execute();
			});
			DeleteRequest->ProcessRequest();
		});

		LatentIt("should reject mismatched Mcp-Protocol-Version header", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("does not match negotiated version"), EAutomationExpectedErrorFlags::Contains);
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				// Send tools/list with a wrong protocol version header
				FHttpModule& HttpModule = FHttpModule::Get();
				TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
				HttpRequest->SetURL(GetTestBaseUrl());
				HttpRequest->SetVerb(TEXT("POST"));
				HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
				HttpRequest->SetHeader(TEXT("Mcp-Session-Id"), SessionId);
				HttpRequest->SetHeader(TEXT("Mcp-Protocol-Version"), TEXT("1999-01-01"));
				HttpRequest->SetContentAsString(JsonObjectToString(MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(10))));
				HttpRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ErrorObject;
							TestTrue("Should have error for version mismatch", JsonResponse->TryGetObjectField(TEXT("error"), ErrorObject));
						}
					}
					CleanupSession(Done);
				});
				HttpRequest->ProcessRequest();
			}));
		});

		LatentIt("should accept matching Mcp-Protocol-Version header", [this](const FDoneDelegate& Done)
		{
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}

				// Send tools/list with the correct protocol version header
				FHttpModule& HttpModule = FHttpModule::Get();
				TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
				HttpRequest->SetURL(GetTestBaseUrl());
				HttpRequest->SetVerb(TEXT("POST"));
				HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
				HttpRequest->SetHeader(TEXT("Mcp-Session-Id"), SessionId);
				HttpRequest->SetHeader(TEXT("Mcp-Protocol-Version"), UE::ModelContextProtocol::ProtocolVersion);
				HttpRequest->SetContentAsString(JsonObjectToString(MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(10))));
				HttpRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							TestTrue("Should have result (no error)", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject));
						}
					}
					CleanupSession(Done);
				});
				HttpRequest->ProcessRequest();
			}));
		});
	});

	Describe("Origin Validation", [this]()
	{
		LatentIt("should reject request with disallowed Origin header", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Rejected request with disallowed Origin"), EAutomationExpectedErrorFlags::Contains);
			FHttpModule& HttpModule = FHttpModule::Get();
			TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
			HttpRequest->SetURL(GetTestBaseUrl());
			HttpRequest->SetVerb(TEXT("POST"));
			HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
			HttpRequest->SetHeader(TEXT("Origin"), TEXT("http://evil.example.com"));
			HttpRequest->SetContentAsString(JsonObjectToString(MakeJsonRpcRequest(TEXT("ping"))));
			HttpRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 403 Forbidden for disallowed Origin", Response->GetResponseCode(), 403);
				}
				Done.Execute();
			});
			HttpRequest->ProcessRequest();
		});

		LatentIt("should accept request with localhost Origin header", [this](const FDoneDelegate& Done)
		{
			FHttpModule& HttpModule = FHttpModule::Get();
			TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
			HttpRequest->SetURL(GetTestBaseUrl());
			HttpRequest->SetVerb(TEXT("POST"));
			HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
			HttpRequest->SetHeader(TEXT("Origin"), TEXT("http://localhost:3000"));
			HttpRequest->SetContentAsString(JsonObjectToString(MakeJsonRpcRequest(TEXT("ping"))));
			HttpRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestTrue("Should not return 403 for localhost Origin", Response->GetResponseCode() != 403);
				}
				Done.Execute();
			});
			HttpRequest->ProcessRequest();
		});

		LatentIt("should accept request with 127.0.0.1 Origin header", [this](const FDoneDelegate& Done)
		{
			FHttpModule& HttpModule = FHttpModule::Get();
			TSharedRef<IHttpRequest> HttpRequest = HttpModule.CreateRequest();
			HttpRequest->SetURL(GetTestBaseUrl());
			HttpRequest->SetVerb(TEXT("POST"));
			HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
			HttpRequest->SetHeader(TEXT("Origin"), TEXT("http://127.0.0.1:8080"));
			HttpRequest->SetContentAsString(JsonObjectToString(MakeJsonRpcRequest(TEXT("ping"))));
			HttpRequest->OnProcessRequestComplete().BindLambda([this, Done](FHttpRequestPtr, FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestTrue("Should not return 403 for 127.0.0.1 Origin", Response->GetResponseCode() != 403);
				}
				Done.Execute();
			});
			HttpRequest->ProcessRequest();
		});

		LatentIt("should accept request with no Origin header", [this](const FDoneDelegate& Done)
		{
			// No Origin header is the default for non-browser clients — verify it's accepted
			TSharedRef<FJsonObject> PingRequest = MakeJsonRpcRequest(TEXT("ping"));
			SendJsonRpcRequest(PingRequest, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestTrue("Should not return 403 when no Origin header is present", Response->GetResponseCode() != 403);
				}
				Done.Execute();
			});
		});
	});

	Describe("Pagination", [this]()
	{
		LatentIt("should return all tools when pagination is disabled", [this](const FDoneDelegate& Done)
		{
			const int32 SavedPageSize = UE::ModelContextProtocol::PaginationPageSize;
			UE::ModelContextProtocol::PaginationPageSize = 0;

			InitializeSession(FDoneDelegate::CreateLambda([this, Done, SavedPageSize]()
			{
				if (SessionId.IsEmpty())
				{
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					Done.Execute();
					return;
				}

				MockTool->Name = TEXT("test_pagination_disabled_tool");
				MockTool->Description = TEXT("Tool for pagination disabled test");
				if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
				{
					Module->AddTool(MockTool.ToSharedRef());
				}

				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(20));
				SendJsonRpcRequest(ListRequest, [this, Done, SavedPageSize](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ToolsArray;
								TestTrue("Should have tools array", (*ResultObject)->TryGetArrayField(TEXT("tools"), ToolsArray));
								TestFalse("Should not have nextCursor when pagination is disabled", (*ResultObject)->HasField(TEXT("nextCursor")));
							}
						}
					}
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should paginate tools/list with nextCursor", [this](const FDoneDelegate& Done)
		{
			const int32 SavedPageSize = UE::ModelContextProtocol::PaginationPageSize;
			UE::ModelContextProtocol::PaginationPageSize = 2;

			InitializeSession(FDoneDelegate::CreateLambda([this, Done, SavedPageSize]()
			{
				if (SessionId.IsEmpty())
				{
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					Done.Execute();
					return;
				}

				// Register 3 mock tools so pagination kicks in with page size 2
				MockTool->Name = TEXT("test_page_tool_a");
				MockTool->Description = TEXT("Pagination tool A");
				if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
				{
					Module->AddTool(MockTool.ToSharedRef());

					for (int32 Index = 0; Index < 2; ++Index)
					{
						TSharedPtr<FMockModelContextProtocolTool> ExtraTool = MakeShared<FMockModelContextProtocolTool>();
						ExtraTool->Name = FString::Printf(TEXT("test_page_tool_%c"), TEXT('b') + Index);
						ExtraTool->Description = FString::Printf(TEXT("Pagination tool %c"), TEXT('B') + Index);
						Module->AddTool(ExtraTool.ToSharedRef());
						AdditionalMockTools.Add(ExtraTool);
					}
				}

				// First page request
				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(21));
				SendJsonRpcRequest(ListRequest, [this, Done, SavedPageSize](FHttpResponsePtr Response, bool bConnected)
				{
					if (!TestTrue("Should connect", bConnected) || !TestTrue("Should have response", Response.IsValid()))
					{
						UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
						CleanupSession(Done);
						return;
					}

					TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
					if (!TestTrue("Should parse response", JsonResponse.IsValid()))
					{
						UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
						CleanupSession(Done);
						return;
					}

					const TSharedPtr<FJsonObject>* ResultObject;
					if (!TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
					{
						UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
						CleanupSession(Done);
						return;
					}

					const TArray<TSharedPtr<FJsonValue>>* ToolsArray;
					if (TestTrue("Should have tools array", (*ResultObject)->TryGetArrayField(TEXT("tools"), ToolsArray)))
					{
						TestEqual("First page should have page size items", ToolsArray->Num(), 2);
					}

					FString NextCursor;
					if (!TestTrue("First page should have nextCursor", (*ResultObject)->TryGetStringField(TEXT("nextCursor"), NextCursor)))
					{
						UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
						CleanupSession(Done);
						return;
					}

					// Follow the cursor to get the next page
					TSharedRef<FJsonObject> NextPageParams = MakeShared<FJsonObject>();
					NextPageParams->SetStringField(TEXT("cursor"), NextCursor);
					TSharedRef<FJsonObject> NextPageRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(22), NextPageParams);
					SendJsonRpcRequest(NextPageRequest, [this, Done, SavedPageSize](FHttpResponsePtr NextResponse, bool bNextConnected)
					{
						if (TestTrue("Should connect for next page", bNextConnected) && TestTrue("Should have next page response", NextResponse.IsValid()))
						{
							TSharedPtr<FJsonObject> NextJsonResponse = ParseJsonResponse(NextResponse);
							if (TestTrue("Should parse next page response", NextJsonResponse.IsValid()))
							{
								const TSharedPtr<FJsonObject>* NextResultObject;
								if (TestTrue("Should have next page result", NextJsonResponse->TryGetObjectField(TEXT("result"), NextResultObject)))
								{
									const TArray<TSharedPtr<FJsonValue>>* NextToolsArray;
									if (TestTrue("Should have tools in next page", (*NextResultObject)->TryGetArrayField(TEXT("tools"), NextToolsArray)))
									{
										// ToolsetRegistry may contribute additional tools in editor builds,
										// so we cannot assert the exact count of the last page.
										TestTrue("Next page should have items", NextToolsArray->Num() > 0);
										TestTrue("Next page should respect page size", NextToolsArray->Num() <= 2);
									}
								}
							}
						}
						UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
						CleanupSession(Done);
					}, SessionId);
				}, SessionId);
			}));
		});

		LatentIt("should paginate resources/list", [this](const FDoneDelegate& Done)
		{
			const int32 SavedPageSize = UE::ModelContextProtocol::PaginationPageSize;
			UE::ModelContextProtocol::PaginationPageSize = 1;

			InitializeSession(FDoneDelegate::CreateLambda([this, Done, SavedPageSize]()
			{
				if (SessionId.IsEmpty())
				{
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					Done.Execute();
					return;
				}

				// Register 2 resources so pagination kicks in with page size 1
				MockResourceProvider->AddTextResource(
					TEXT("file:///test_page_resource_a.txt"),
					TEXT("content A"),
					TOptional<FString>(TEXT("test_page_resource_a")),
					TOptional<FString>(TEXT("Page Resource A")),
					TOptional<FString>(TEXT("A test resource for pagination")),
					TOptional<FString>(TEXT("text/plain")));
				MockResourceProvider->AddTextResource(
					TEXT("file:///test_page_resource_b.txt"),
					TEXT("content B"),
					TOptional<FString>(TEXT("test_page_resource_b")),
					TOptional<FString>(TEXT("Page Resource B")),
					TOptional<FString>(TEXT("Another test resource for pagination")),
					TOptional<FString>(TEXT("text/plain")));
				if (IModelContextProtocolModule* Module = IModelContextProtocolModule::Get())
				{
					Module->AddResourceProvider(MockResourceProvider.ToSharedRef());
				}

				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("resources/list"), MakeShared<FJsonValueNumber>(23));
				SendJsonRpcRequest(ListRequest, [this, Done, SavedPageSize](FHttpResponsePtr Response, bool bConnected)
				{
					if (!TestTrue("Should connect", bConnected) || !TestTrue("Should have response", Response.IsValid()))
					{
						UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
						CleanupSession(Done);
						return;
					}

					TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
					if (!TestTrue("Should parse response", JsonResponse.IsValid()))
					{
						UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
						CleanupSession(Done);
						return;
					}

					const TSharedPtr<FJsonObject>* ResultObject;
					if (!TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
					{
						UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
						CleanupSession(Done);
						return;
					}

					const TArray<TSharedPtr<FJsonValue>>* ResourcesArray;
					if (TestTrue("Should have resources array", (*ResultObject)->TryGetArrayField(TEXT("resources"), ResourcesArray)))
					{
						TestEqual("First page should have 1 resource", ResourcesArray->Num(), 1);
					}

					FString NextCursor;
					if (TestTrue("Should have nextCursor for resources", (*ResultObject)->TryGetStringField(TEXT("nextCursor"), NextCursor)))
					{
						// Follow the cursor
						TSharedRef<FJsonObject> NextPageParams = MakeShared<FJsonObject>();
						NextPageParams->SetStringField(TEXT("cursor"), NextCursor);
						TSharedRef<FJsonObject> NextPageRequest = MakeJsonRpcRequest(TEXT("resources/list"), MakeShared<FJsonValueNumber>(24), NextPageParams);
						SendJsonRpcRequest(NextPageRequest, [this, Done, SavedPageSize](FHttpResponsePtr NextResponse, bool bNextConnected)
						{
							if (TestTrue("Should connect for next page", bNextConnected) && TestTrue("Should have next page response", NextResponse.IsValid()))
							{
								TSharedPtr<FJsonObject> NextJsonResponse = ParseJsonResponse(NextResponse);
								if (TestTrue("Should parse next page response", NextJsonResponse.IsValid()))
								{
									const TSharedPtr<FJsonObject>* NextResultObject;
									if (TestTrue("Should have next page result", NextJsonResponse->TryGetObjectField(TEXT("result"), NextResultObject)))
									{
										const TArray<TSharedPtr<FJsonValue>>* NextResourcesArray;
										if (TestTrue("Should have resources in next page", (*NextResultObject)->TryGetArrayField(TEXT("resources"), NextResourcesArray)))
										{
											TestEqual("Last page should have 1 resource", NextResourcesArray->Num(), 1);
										}
										TestFalse("Last page should not have nextCursor", (*NextResultObject)->HasField(TEXT("nextCursor")));
									}
								}
							}
							UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
							CleanupSession(Done);
						}, SessionId);
					}
					else
					{
						UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
						CleanupSession(Done);
					}
				}, SessionId);
			}));
		});

		LatentIt("should return error for invalid Base64 cursor", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Invalid pagination cursor"), EAutomationExpectedErrorFlags::Contains);
			const int32 SavedPageSize = UE::ModelContextProtocol::PaginationPageSize;
			UE::ModelContextProtocol::PaginationPageSize = 2;

			InitializeSession(FDoneDelegate::CreateLambda([this, Done, SavedPageSize]()
			{
				if (SessionId.IsEmpty())
				{
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					Done.Execute();
					return;
				}

				TSharedRef<FJsonObject> ListParams = MakeShared<FJsonObject>();
				ListParams->SetStringField(TEXT("cursor"), TEXT("not-valid-base64-cursor!!!"));
				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(25), ListParams);
				SendJsonRpcRequest(ListRequest, [this, Done, SavedPageSize](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ErrorObject;
							if (TestTrue("Should have error for invalid cursor", JsonResponse->TryGetObjectField(TEXT("error"), ErrorObject)))
							{
								int32 ErrorCode = 0;
								(*ErrorObject)->TryGetNumberField(TEXT("code"), ErrorCode);
								TestEqual("Error code should be InvalidParams (-32602)", ErrorCode, -32602);
							}
						}
					}
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should return error for cursor that decodes to non-numeric text", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Invalid pagination cursor"), EAutomationExpectedErrorFlags::Contains);
			const int32 SavedPageSize = UE::ModelContextProtocol::PaginationPageSize;
			UE::ModelContextProtocol::PaginationPageSize = 2;

			InitializeSession(FDoneDelegate::CreateLambda([this, Done, SavedPageSize]()
			{
				if (SessionId.IsEmpty())
				{
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					Done.Execute();
					return;
				}

				// Valid Base64 that decodes to non-numeric text
				TSharedRef<FJsonObject> ListParams = MakeShared<FJsonObject>();
				ListParams->SetStringField(TEXT("cursor"), FBase64::Encode(TEXT("abc")));
				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(26), ListParams);
				SendJsonRpcRequest(ListRequest, [this, Done, SavedPageSize](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ErrorObject;
							if (TestTrue("Should have error for non-numeric cursor", JsonResponse->TryGetObjectField(TEXT("error"), ErrorObject)))
							{
								int32 ErrorCode = 0;
								(*ErrorObject)->TryGetNumberField(TEXT("code"), ErrorCode);
								TestEqual("Error code should be InvalidParams (-32602)", ErrorCode, -32602);
							}
						}
					}
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					CleanupSession(Done);
				}, SessionId);
			}));
		});

		LatentIt("should return empty page for cursor past end of list", [this](const FDoneDelegate& Done)
		{
			const int32 SavedPageSize = UE::ModelContextProtocol::PaginationPageSize;
			UE::ModelContextProtocol::PaginationPageSize = 2;

			InitializeSession(FDoneDelegate::CreateLambda([this, Done, SavedPageSize]()
			{
				if (SessionId.IsEmpty())
				{
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					Done.Execute();
					return;
				}

				// Use a cursor with an offset far beyond any possible tool count.
				// ToolsetRegistry may contribute additional tools in editor builds,
				// so we use MAX_int32 to guarantee the offset exceeds the total.
				const FString FarCursor = FBase64::Encode(FString::FromInt(MAX_int32));
				TSharedRef<FJsonObject> ListParams = MakeShared<FJsonObject>();
				ListParams->SetStringField(TEXT("cursor"), FarCursor);
				TSharedRef<FJsonObject> ListRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(27), ListParams);
				SendJsonRpcRequest(ListRequest, [this, Done, SavedPageSize](FHttpResponsePtr Response, bool bConnected)
				{
					if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
					{
						TSharedPtr<FJsonObject> JsonResponse = ParseJsonResponse(Response);
						if (TestTrue("Should parse response", JsonResponse.IsValid()))
						{
							const TSharedPtr<FJsonObject>* ResultObject;
							if (TestTrue("Should have result", JsonResponse->TryGetObjectField(TEXT("result"), ResultObject)))
							{
								const TArray<TSharedPtr<FJsonValue>>* ToolsArray;
								if (TestTrue("Should have tools array", (*ResultObject)->TryGetArrayField(TEXT("tools"), ToolsArray)))
								{
									TestEqual("Should return empty tools array for cursor past end", ToolsArray->Num(), 0);
								}
								TestFalse("Should not have nextCursor for cursor past end", (*ResultObject)->HasField(TEXT("nextCursor")));
							}
						}
					}
					UE::ModelContextProtocol::PaginationPageSize = SavedPageSize;
					CleanupSession(Done);
				}, SessionId);
			}));
		});
	});

	Describe("Session Validation", [this]()
	{
		LatentIt("tools/call without Mcp-Session-Id header returns 400", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Missing required Mcp-Session-Id header"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Params = MakeShared<FJsonObject>();
			Params->SetStringField(TEXT("name"), TEXT("test_tool"));
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(1), Params);
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 400 for missing session header", Response->GetResponseCode(), 400);
				}
				Done.Execute();
			});
		});

		LatentIt("tools/call with unknown Mcp-Session-Id returns 404", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Unknown session id"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Params = MakeShared<FJsonObject>();
			Params->SetStringField(TEXT("name"), TEXT("test_tool"));
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("tools/call"), MakeShared<FJsonValueNumber>(1), Params);
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 404 for unknown session id", Response->GetResponseCode(), 404);
				}
				Done.Execute();
			}, TEXT("test-stale-session-id"));
		});

		LatentIt("tools/list without Mcp-Session-Id header returns 400", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Missing required Mcp-Session-Id header"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(1));
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 400 for missing session header", Response->GetResponseCode(), 400);
				}
				Done.Execute();
			});
		});

		LatentIt("tools/list with unknown Mcp-Session-Id returns 404", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Unknown session id"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(1));
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 404 for unknown session id", Response->GetResponseCode(), 404);
				}
				Done.Execute();
			}, TEXT("test-stale-session-id"));
		});

		LatentIt("resources/list without Mcp-Session-Id header returns 400", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Missing required Mcp-Session-Id header"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("resources/list"), MakeShared<FJsonValueNumber>(1));
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 400 for missing session header", Response->GetResponseCode(), 400);
				}
				Done.Execute();
			});
		});

		LatentIt("resources/list with unknown Mcp-Session-Id returns 404", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Unknown session id"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("resources/list"), MakeShared<FJsonValueNumber>(1));
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 404 for unknown session id", Response->GetResponseCode(), 404);
				}
				Done.Execute();
			}, TEXT("test-stale-session-id"));
		});

		LatentIt("resources/read without Mcp-Session-Id header returns 400", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Missing required Mcp-Session-Id header"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Params = MakeShared<FJsonObject>();
			Params->SetStringField(TEXT("uri"), TEXT("mcp://test-resource"));
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("resources/read"), MakeShared<FJsonValueNumber>(1), Params);
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 400 for missing session header", Response->GetResponseCode(), 400);
				}
				Done.Execute();
			});
		});

		LatentIt("resources/read with unknown Mcp-Session-Id returns 404", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Unknown session id"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Params = MakeShared<FJsonObject>();
			Params->SetStringField(TEXT("uri"), TEXT("mcp://test-resource"));
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("resources/read"), MakeShared<FJsonValueNumber>(1), Params);
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 404 for unknown session id", Response->GetResponseCode(), 404);
				}
				Done.Execute();
			}, TEXT("test-stale-session-id"));
		});

		LatentIt("notifications/initialized without Mcp-Session-Id header returns 400", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Missing required Mcp-Session-Id header"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Notification = MakeJsonRpcNotification(TEXT("notifications/initialized"));
			SendJsonRpcRequest(Notification, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 400 for missing session header", Response->GetResponseCode(), 400);
				}
				Done.Execute();
			});
		});

		LatentIt("notifications/initialized with unknown Mcp-Session-Id returns 404", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Unknown session id"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> Notification = MakeJsonRpcNotification(TEXT("notifications/initialized"));
			SendJsonRpcRequest(Notification, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 404 for unknown session id", Response->GetResponseCode(), 404);
				}
				Done.Execute();
			}, TEXT("test-stale-session-id"));
		});

		LatentIt("notifications/cancelled without Mcp-Session-Id header returns 400", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Missing required Mcp-Session-Id header"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> CancelParams = MakeShared<FJsonObject>();
			CancelParams->SetField(TEXT("requestId"), MakeShared<FJsonValueNumber>(999));
			TSharedRef<FJsonObject> Notification = MakeJsonRpcNotification(TEXT("notifications/cancelled"), CancelParams);
			SendJsonRpcRequest(Notification, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 400 for missing session header (was silent 202)", Response->GetResponseCode(), 400);
				}
				Done.Execute();
			});
		});

		LatentIt("notifications/cancelled with unknown Mcp-Session-Id returns 404", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Unknown session id"), EAutomationExpectedErrorFlags::Contains);
			TSharedRef<FJsonObject> CancelParams = MakeShared<FJsonObject>();
			CancelParams->SetField(TEXT("requestId"), MakeShared<FJsonValueNumber>(999));
			TSharedRef<FJsonObject> Notification = MakeJsonRpcNotification(TEXT("notifications/cancelled"), CancelParams);
			SendJsonRpcRequest(Notification, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("Should return 404 for unknown session id", Response->GetResponseCode(), 404);
				}
				Done.Execute();
			}, TEXT("test-stale-session-id"));
		});

		LatentIt("stale id from prior session is rejected with 404, fresh session works", [this](const FDoneDelegate& Done)
		{
			AddExpectedError(TEXT("Unknown session id"), EAutomationExpectedErrorFlags::Contains);
			InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
			{
				if (SessionId.IsEmpty())
				{
					Done.Execute();
					return;
				}
				const FString StaleId = SessionId;

				FHttpModule& HttpModule = FHttpModule::Get();
				TSharedRef<IHttpRequest> DeleteRequest = HttpModule.CreateRequest();
				DeleteRequest->SetURL(GetTestBaseUrl());
				DeleteRequest->SetVerb(TEXT("DELETE"));
				DeleteRequest->SetHeader(TEXT("Mcp-Session-Id"), StaleId);
				DeleteRequest->OnProcessRequestComplete().BindLambda(
					[this, Done, StaleId](FHttpRequestPtr, FHttpResponsePtr DeleteResponse, bool bDeleteConnected)
				{
					SessionId.Reset();
					TestTrue("DELETE should succeed", bDeleteConnected && DeleteResponse.IsValid()
						&& DeleteResponse->GetResponseCode() == 202);

					TSharedRef<FJsonObject> StaleRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(2));
					SendJsonRpcRequest(StaleRequest, [this, Done](FHttpResponsePtr StaleResponse, bool bStaleConnected)
					{
						if (TestTrue("Stale request should connect", bStaleConnected)
							&& TestTrue("Stale request should have response", StaleResponse.IsValid()))
						{
							TestEqual("Stale id should be 404", StaleResponse->GetResponseCode(), 404);
						}

						InitializeSession(FDoneDelegate::CreateLambda([this, Done]()
						{
							if (SessionId.IsEmpty())
							{
								Done.Execute();
								return;
							}
							TSharedRef<FJsonObject> FreshRequest = MakeJsonRpcRequest(TEXT("tools/list"), MakeShared<FJsonValueNumber>(3));
							SendJsonRpcRequest(FreshRequest, [this, Done](FHttpResponsePtr FreshResponse, bool bFreshConnected)
							{
								if (TestTrue("Fresh request should connect", bFreshConnected)
									&& TestTrue("Fresh request should have response", FreshResponse.IsValid()))
								{
									TestEqual("Fresh id should succeed", FreshResponse->GetResponseCode(), 200);
								}
								CleanupSession(Done);
							}, SessionId);
						}));
					}, StaleId);
				});
				DeleteRequest->ProcessRequest();
			}));
		});

		LatentIt("ping without Mcp-Session-Id header succeeds (exempt method)", [this](const FDoneDelegate& Done)
		{
			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("ping"), MakeShared<FJsonValueNumber>(1));
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("ping is exempt; should return 200", Response->GetResponseCode(), 200);
				}
				Done.Execute();
			});
		});

		LatentIt("initialize without Mcp-Session-Id header succeeds and assigns one (exempt method)", [this](const FDoneDelegate& Done)
		{
			TSharedRef<FJsonObject> InitParams = MakeShared<FJsonObject>();
			InitParams->SetStringField(TEXT("protocolVersion"), UE::ModelContextProtocol::ProtocolVersion);
			TSharedRef<FJsonObject> ClientInfo = MakeShared<FJsonObject>();
			ClientInfo->SetStringField(TEXT("name"), TEXT("TestClient"));
			ClientInfo->SetStringField(TEXT("version"), TEXT("1.0"));
			InitParams->SetObjectField(TEXT("clientInfo"), ClientInfo);
			InitParams->SetObjectField(TEXT("capabilities"), MakeShared<FJsonObject>());

			TSharedRef<FJsonObject> Request = MakeJsonRpcRequest(TEXT("initialize"), MakeShared<FJsonValueNumber>(1), InitParams);
			SendJsonRpcRequest(Request, [this, Done](FHttpResponsePtr Response, bool bConnected)
			{
				if (TestTrue("Should connect", bConnected) && TestTrue("Should have response", Response.IsValid()))
				{
					TestEqual("initialize is exempt; should return 200", Response->GetResponseCode(), 200);
					const FString AssignedId = Response->GetHeader(TEXT("Mcp-Session-Id"));
					TestFalse("Should assign Mcp-Session-Id header", AssignedId.IsEmpty());
					SessionId = AssignedId;
				}
				CleanupSession(Done);
			});
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
