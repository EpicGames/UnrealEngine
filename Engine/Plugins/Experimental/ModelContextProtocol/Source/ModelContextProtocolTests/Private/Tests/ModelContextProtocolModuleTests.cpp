// Copyright Epic Games, Inc. All Rights Reserved.

#include "IModelContextProtocolModule.h"
#include "IModelContextProtocolTool.h"
#include "IModelContextProtocolResourceProvider.h"
#include "Mocks/MockModelContextProtocolTool.h"
#include "Mocks/MockModelContextProtocolResourceProvider.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolModuleTests, "AI.ModelContextProtocol.Module", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	IModelContextProtocolModule* Module;
	TSharedPtr<FMockModelContextProtocolTool> MockTool;
	TSharedPtr<FMockModelContextProtocolResourceProvider> MockProvider;
	TArray<TSharedRef<IModelContextProtocolTool>> ToolsToCleanUp;
	TArray<TSharedRef<IModelContextProtocolResourceProvider>> ProvidersToCleanUp;
END_DEFINE_SPEC(FModelContextProtocolModuleTests)

void FModelContextProtocolModuleTests::Define()
{
	BeforeEach([this]()
	{
		Module = IModelContextProtocolModule::Get();
		MockTool = MakeShared<FMockModelContextProtocolTool>();
		MockProvider = MakeShared<FMockModelContextProtocolResourceProvider>();
		ToolsToCleanUp.Reset();
		ProvidersToCleanUp.Reset();
	});

	AfterEach([this]()
	{
		if (Module)
		{
			for (const TSharedRef<IModelContextProtocolTool>& Tool : ToolsToCleanUp)
			{
				Module->RemoveTool(Tool);
			}
			for (const TSharedRef<IModelContextProtocolResourceProvider>& Provider : ProvidersToCleanUp)
			{
				Module->RemoveResourceProvider(Provider);
			}
		}
		MockTool.Reset();
		MockProvider.Reset();
	});

	Describe("Default inputSchema", [this]()
	{
		It("should return a valid JSON Schema object from the base class GetInputJsonSchema", [this]()
		{
			// A minimal tool that does not override GetInputJsonSchema — it should inherit the base class default.
			struct FMinimalTool : IModelContextProtocolTool
			{
				virtual FString GetName() const override { return TEXT("test_minimal"); }
				virtual FString GetDescription() const override { return TEXT(""); }
			};

			FMinimalTool Tool;
			TSharedPtr<FJsonObject> Schema = Tool.GetInputJsonSchema();
			if (TestTrue("Default GetInputJsonSchema should return a valid object", Schema.IsValid()))
			{
				FString TypeValue;
				if (TestTrue("Schema should have a 'type' field", Schema->TryGetStringField(TEXT("type"), TypeValue)))
				{
					TestEqual("Schema type should be 'object'", TypeValue, TEXT("object"));
				}
			}
		});
	});

	Describe("Tool management", [this]()
	{
		It("should register a tool via AddTool and retrieve it via FindTool", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			MockTool->Name = TEXT("test_add_tool");
			MockTool->Description = TEXT("A test tool that adds two numbers together");
			MockTool->InputSchema = FMockModelContextProtocolTool::MakeTestInputSchema(
				{{TEXT("a"), TEXT("number")}, {TEXT("b"), TEXT("number")}},
				{TEXT("a"), TEXT("b")});
			MockTool->OutputSchema = FMockModelContextProtocolTool::MakeTestInputSchema(
				{{TEXT("sum"), TEXT("number")}});
			TSharedRef<IModelContextProtocolTool> ToolRef = MockTool.ToSharedRef();
			TestTrue("AddTool should succeed", Module->AddTool(ToolRef));
			ToolsToCleanUp.Add(ToolRef);
			TSharedPtr<IModelContextProtocolTool> Found = Module->FindTool(TEXT("test_add_tool"));
			if (!TestTrue("FindTool should return the tool", Found.IsValid())) { return; }
			TestEqual("Found tool should be our mock", Found.Get(), static_cast<IModelContextProtocolTool*>(MockTool.Get()));
			TestEqual("Found tool description should match", Found->GetDescription(), TEXT("A test tool that adds two numbers together"));
			TestTrue("Found tool should have input schema", Found->GetInputJsonSchema().IsValid());
			TSharedPtr<FJsonObject> FoundOutputSchema = Found->GetOutputJsonSchema();
			if (TestTrue("Found tool should have output schema", FoundOutputSchema.IsValid()))
			{
				FString SchemaType;
				TestTrue("Output schema should have type", FoundOutputSchema->TryGetStringField(TEXT("type"), SchemaType));
				TestEqual("Output schema type should be object", SchemaType, TEXT("object"));
				const TSharedPtr<FJsonObject>* PropertiesObject;
				if (TestTrue("Output schema should have properties", FoundOutputSchema->TryGetObjectField(TEXT("properties"), PropertiesObject)))
				{
					TestTrue("Output schema should have sum property", (*PropertiesObject)->HasField(TEXT("sum")));
				}
			}
		});

		It("should find tools case-insensitively", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			MockTool->Name = TEXT("test_case_sensitive");
			TSharedRef<IModelContextProtocolTool> ToolRef = MockTool.ToSharedRef();
			Module->AddTool(ToolRef);
			ToolsToCleanUp.Add(ToolRef);
			TSharedPtr<IModelContextProtocolTool> Found = Module->FindTool(TEXT("TEST_CASE_SENSITIVE"));
			TestTrue("Case-insensitive lookup should work", Found.IsValid());
		});

		It("should return nullptr for unknown tool names", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			TSharedPtr<IModelContextProtocolTool> Found = Module->FindTool(TEXT("test_nonexistent_tool"));
			TestFalse("Should return nullptr for unknown", Found.IsValid());
		});

		It("should reject duplicate tool names", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			MockTool->Name = TEXT("test_duplicate_tool");
			TSharedRef<IModelContextProtocolTool> ToolRef = MockTool.ToSharedRef();
			Module->AddTool(ToolRef);
			ToolsToCleanUp.Add(ToolRef);

			TSharedPtr<FMockModelContextProtocolTool> DuplicateTool = MakeShared<FMockModelContextProtocolTool>();
			DuplicateTool->Name = TEXT("test_duplicate_tool");
			AddExpectedError(TEXT("is already registered"), EAutomationExpectedErrorFlags::Contains);
			TestFalse("AddTool should reject duplicate name", Module->AddTool(DuplicateTool.ToSharedRef()));
		});

		It("should remove a tool via RemoveTool", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			MockTool->Name = TEXT("test_remove_tool");
			TSharedRef<IModelContextProtocolTool> ToolRef = MockTool.ToSharedRef();
			Module->AddTool(ToolRef);
			TestTrue("RemoveTool should succeed", Module->RemoveTool(ToolRef));
			TSharedPtr<IModelContextProtocolTool> Found = Module->FindTool(TEXT("test_remove_tool"));
			TestFalse("Tool should not be found after removal", Found.IsValid());
		});

		It("should return false when removing an unregistered tool", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			TSharedPtr<FMockModelContextProtocolTool> UnregisteredTool = MakeShared<FMockModelContextProtocolTool>();
			UnregisteredTool->Name = TEXT("test_unregistered_tool");
			TestFalse("RemoveTool should return false for unregistered", Module->RemoveTool(UnregisteredTool.ToSharedRef()));
		});

		It("should return tools via GetTools", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			const int32 InitialCount = Module->GetTools().Num();
			MockTool->Name = TEXT("test_get_tools");
			TSharedRef<IModelContextProtocolTool> ToolRef = MockTool.ToSharedRef();
			Module->AddTool(ToolRef);
			ToolsToCleanUp.Add(ToolRef);
			TestEqual("GetTools count should increase by 1", Module->GetTools().Num(), InitialCount + 1);
		});
	});

	Describe("RefreshTools", [this]()
	{
		It("should clear all registered tools and broadcast OnRefreshTools", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			MockTool->Name = TEXT("test_refresh_tool");
			TSharedRef<IModelContextProtocolTool> ToolRef = MockTool.ToSharedRef();
			Module->AddTool(ToolRef);

			bool bDelegateFired = false;
			FDelegateHandle Handle = Module->OnRefreshTools().AddLambda([&bDelegateFired]()
			{
				bDelegateFired = true;
			});

			Module->RefreshTools();

			TestTrue("OnRefreshTools delegate should have fired", bDelegateFired);
			TestFalse("Tool should not be found after refresh", Module->FindTool(TEXT("test_refresh_tool")).IsValid());

			Module->OnRefreshTools().Remove(Handle);
		});

		It("should allow re-registration after refresh", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			MockTool->Name = TEXT("test_reregister_tool");
			TSharedRef<IModelContextProtocolTool> ToolRef = MockTool.ToSharedRef();
			Module->AddTool(ToolRef);
			Module->RefreshTools();
			TestTrue("Should be able to re-add after refresh", Module->AddTool(ToolRef));
			ToolsToCleanUp.Add(ToolRef);
		});
	});

	Describe("Resource provider management", [this]()
	{
		It("should register a resource provider via AddResourceProvider", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			const int32 InitialCount = Module->GetResourceProviders().Num();
			TSharedRef<IModelContextProtocolResourceProvider> ProviderRef = MockProvider.ToSharedRef();
			Module->AddResourceProvider(ProviderRef);
			ProvidersToCleanUp.Add(ProviderRef);
			TestEqual("Provider count should increase by 1", Module->GetResourceProviders().Num(), InitialCount + 1);
		});

		It("should remove a resource provider via RemoveResourceProvider", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			TSharedRef<IModelContextProtocolResourceProvider> ProviderRef = MockProvider.ToSharedRef();
			Module->AddResourceProvider(ProviderRef);
			TestTrue("RemoveResourceProvider should succeed", Module->RemoveResourceProvider(ProviderRef));
		});

		It("should return false when removing an unregistered provider", [this]()
		{
			if (!TestNotNull("Module should exist", Module)) { return; }
			TSharedPtr<FMockModelContextProtocolResourceProvider> UnregisteredProvider = MakeShared<FMockModelContextProtocolResourceProvider>();
			TestFalse("RemoveResourceProvider should return false for unregistered", Module->RemoveResourceProvider(UnregisteredProvider.ToSharedRef()));
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
