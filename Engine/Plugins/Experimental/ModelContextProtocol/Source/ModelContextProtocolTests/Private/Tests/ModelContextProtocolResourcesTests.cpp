// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolResources.h"
#include "Mocks/MockModelContextProtocolResourceProvider.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolResourcesTests, "AI.ModelContextProtocol.Resources", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	TSharedPtr<FMockModelContextProtocolResourceProvider> MockProvider;
END_DEFINE_SPEC(FModelContextProtocolResourcesTests)

void FModelContextProtocolResourcesTests::Define()
{
	BeforeEach([this]()
	{
		MockProvider = MakeShared<FMockModelContextProtocolResourceProvider>();
	});

	AfterEach([this]()
	{
		MockProvider.Reset();
	});

	Describe("FModelContextProtocolResourceDescriptor", [this]()
	{
		It("should store and return URI", [this]()
		{
			FModelContextProtocolResourceDescriptor Descriptor(TEXT("file:///test_document.txt"));
			TestEqual("URI should match", Descriptor.GetUri(), TEXT("file:///test_document.txt"));
		});

		It("should include optional fields in JSON when set with correct values", [this]()
		{
			FModelContextProtocolResourceDescriptor Descriptor(
				TEXT("file:///test_document.txt"),
				TOptional<FString>(TEXT("test_document")),
				TOptional<FString>(TEXT("Test Document")),
				TOptional<FString>(TEXT("A test resource for descriptor validation")),
				TOptional<FString>(TEXT("text/plain")));
			TSharedRef<FJsonObject> JsonObject = Descriptor.GetJsonObject();
			FString Name, Title, Description, MimeType;
			if (TestTrue("Should have name", JsonObject->TryGetStringField(TEXT("name"), Name)))
			{
				TestEqual("Name should match", Name, TEXT("test_document"));
			}
			if (TestTrue("Should have title", JsonObject->TryGetStringField(TEXT("title"), Title)))
			{
				TestEqual("Title should match", Title, TEXT("Test Document"));
			}
			if (TestTrue("Should have description", JsonObject->TryGetStringField(TEXT("description"), Description)))
			{
				TestEqual("Description should match", Description, TEXT("A test resource for descriptor validation"));
			}
			if (TestTrue("Should have mimeType", JsonObject->TryGetStringField(TEXT("mimeType"), MimeType)))
			{
				TestEqual("MimeType should match", MimeType, TEXT("text/plain"));
			}
		});

		It("should omit optional fields when not set", [this]()
		{
			FModelContextProtocolResourceDescriptor Descriptor(TEXT("file:///test_minimal.txt"));
			TSharedRef<FJsonObject> JsonObject = Descriptor.GetJsonObject();
			TestTrue("Should have uri", JsonObject->HasField(TEXT("uri")));
			TestFalse("Should not have name", JsonObject->HasField(TEXT("name")));
			TestFalse("Should not have title", JsonObject->HasField(TEXT("title")));
			TestFalse("Should not have description", JsonObject->HasField(TEXT("description")));
			TestFalse("Should not have mimeType", JsonObject->HasField(TEXT("mimeType")));
		});
	});

	Describe("FModelContextProtocolResource", [this]()
	{
		It("should include text content for string resources", [this]()
		{
			FModelContextProtocolResource Resource(TEXT("file:///test_hello.txt"), FString(TEXT("hello world")));
			TSharedRef<FJsonObject> JsonObject = Resource.GetJsonObject();
			FString Text;
			if (TestTrue("Should have text field", JsonObject->TryGetStringField(TEXT("text"), Text)))
			{
				TestEqual("Text should match", Text, TEXT("hello world"));
			}
			FString Uri;
			if (TestTrue("Should have uri field", JsonObject->TryGetStringField(TEXT("uri"), Uri)))
			{
				TestEqual("URI should match", Uri, TEXT("file:///test_hello.txt"));
			}
		});

		It("should include text content with all optional metadata", [this]()
		{
			FModelContextProtocolResource Resource(
				TEXT("file:///test_readme.md"),
				FString(TEXT("# Test README")),
				TOptional<FString>(TEXT("test_readme")),
				TOptional<FString>(TEXT("Test README")),
				TOptional<FString>(TEXT("text/markdown")));
			TSharedRef<FJsonObject> JsonObject = Resource.GetJsonObject();
			FString Text, Uri, Name, Title, MimeType;
			TestTrue("Should have text", JsonObject->TryGetStringField(TEXT("text"), Text));
			TestEqual("Text should match", Text, TEXT("# Test README"));
			TestTrue("Should have uri", JsonObject->TryGetStringField(TEXT("uri"), Uri));
			TestEqual("URI should match", Uri, TEXT("file:///test_readme.md"));
			if (TestTrue("Should have name", JsonObject->TryGetStringField(TEXT("name"), Name)))
			{
				TestEqual("Name should match", Name, TEXT("test_readme"));
			}
			if (TestTrue("Should have title", JsonObject->TryGetStringField(TEXT("title"), Title)))
			{
				TestEqual("Title should match", Title, TEXT("Test README"));
			}
			if (TestTrue("Should have mimeType", JsonObject->TryGetStringField(TEXT("mimeType"), MimeType)))
			{
				TestEqual("MimeType should match", MimeType, TEXT("text/markdown"));
			}
		});

		It("should base64-encode blob content for binary resources", [this]()
		{
			TArray<uint8> BlobData = { 0x48, 0x65, 0x6C, 0x6C, 0x6F };
			FModelContextProtocolResource Resource(TEXT("file:///test_binary.bin"), TArrayView<uint8>(BlobData));
			TSharedRef<FJsonObject> JsonObject = Resource.GetJsonObject();
			TestTrue("Should have blob field", JsonObject->HasField(TEXT("blob")));
		});

		It("should base64-encode blob content with metadata", [this]()
		{
			TArray<uint8> PngHeader = { 0x89, 0x50, 0x4E, 0x47, 0x0D, 0x0A, 0x1A, 0x0A };
			FModelContextProtocolResource Resource(
				TEXT("file:///test_screenshot.png"),
				TArrayView<uint8>(PngHeader),
				TOptional<FString>(TEXT("test_screenshot")),
				TOptional<FString>(TEXT("Test Screenshot")),
				TOptional<FString>(TEXT("image/png")));
			TSharedRef<FJsonObject> JsonObject = Resource.GetJsonObject();
			FString Blob, Uri, Name, MimeType;
			TestTrue("Should have blob", JsonObject->TryGetStringField(TEXT("blob"), Blob));
			TestFalse("Blob should not be empty", Blob.IsEmpty());
			TestTrue("Should have uri", JsonObject->TryGetStringField(TEXT("uri"), Uri));
			TestEqual("URI should match", Uri, TEXT("file:///test_screenshot.png"));
			if (TestTrue("Should have name", JsonObject->TryGetStringField(TEXT("name"), Name)))
			{
				TestEqual("Name should match", Name, TEXT("test_screenshot"));
			}
			if (TestTrue("Should have mimeType", JsonObject->TryGetStringField(TEXT("mimeType"), MimeType)))
			{
				TestEqual("MimeType should match", MimeType, TEXT("image/png"));
			}
		});
	});

	Describe("FModelContextProtocolResourceDescriptorList", [this]()
	{
		It("should start empty with Num() == 0", [this]()
		{
			FModelContextProtocolResourceDescriptorList List;
			TestEqual("Should start empty", List.Num(), 0);
		});

		It("should track count after adding descriptors", [this]()
		{
			FModelContextProtocolResourceDescriptorList List;
			FModelContextProtocolResourceDescriptor Descriptor1(
				TEXT("file:///test_config.json"),
				TOptional<FString>(TEXT("test_config")),
				TOptional<FString>(TEXT("Test Configuration")),
				TOptional<FString>(TEXT("Test application configuration")),
				TOptional<FString>(TEXT("application/json")));
			FModelContextProtocolResourceDescriptor Descriptor2(
				TEXT("file:///test_data.csv"),
				TOptional<FString>(TEXT("test_data")),
				TOptional<FString>(TEXT("Test Data Export")),
				TOptional<FString>(TEXT("Test exported dataset")),
				TOptional<FString>(TEXT("text/csv")));
			List.Add(Descriptor1, MockProvider.ToSharedRef());
			List.Add(Descriptor2, MockProvider.ToSharedRef());
			TestEqual("Should have 2 descriptors", List.Num(), 2);
		});

		It("should produce valid JSON array via GetJsonArray", [this]()
		{
			FModelContextProtocolResourceDescriptorList List;
			FModelContextProtocolResourceDescriptor Descriptor(TEXT("file:///test_array.txt"));
			List.Add(Descriptor, MockProvider.ToSharedRef());
			TSharedRef<FJsonValueArray> JsonArray = List.GetJsonArray();
			TestEqual("JSON array should have 1 element", JsonArray->AsArray().Num(), 1);
		});

		It("should map URI to provider via FindResourceProvider", [this]()
		{
			FModelContextProtocolResourceDescriptorList List;
			FModelContextProtocolResourceDescriptor Descriptor(TEXT("file:///test_lookup.txt"));
			List.Add(Descriptor, MockProvider.ToSharedRef());
			TSharedPtr<const IModelContextProtocolResourceProvider> FoundProvider = List.FindResourceProvider(TEXT("file:///test_lookup.txt"));
			TestTrue("Should find the provider", FoundProvider.IsValid());
			TestEqual("Should be our mock provider", FoundProvider.Get(), static_cast<const IModelContextProtocolResourceProvider*>(MockProvider.Get()));
		});

		It("should return nullptr for unknown URIs", [this]()
		{
			FModelContextProtocolResourceDescriptorList List;
			TSharedPtr<const IModelContextProtocolResourceProvider> FoundProvider = List.FindResourceProvider(TEXT("file:///test_unknown.txt"));
			TestFalse("Should not find provider for unknown URI", FoundProvider.IsValid());
		});

		It("should clear everything on Reset", [this]()
		{
			FModelContextProtocolResourceDescriptorList List;
			FModelContextProtocolResourceDescriptor Descriptor(TEXT("file:///test_reset.txt"));
			List.Add(Descriptor, MockProvider.ToSharedRef());
			List.Reset();
			TestEqual("Should be empty after Reset", List.Num(), 0);
			TestFalse("Should not find provider after Reset", List.FindResourceProvider(TEXT("file:///test_reset.txt")).IsValid());
		});

		It("should preserve URI-to-provider mapping after ReleaseJsonArray", [this]()
		{
			FModelContextProtocolResourceDescriptorList List;
			FModelContextProtocolResourceDescriptor Descriptor(TEXT("file:///test_release.txt"));
			List.Add(Descriptor, MockProvider.ToSharedRef());
			List.ReleaseJsonArray();
			TSharedPtr<const IModelContextProtocolResourceProvider> FoundProvider = List.FindResourceProvider(TEXT("file:///test_release.txt"));
			TestTrue("Provider mapping should survive ReleaseJsonArray", FoundProvider.IsValid());
		});

		It("should list and read resources through mock provider with full metadata", [this]()
		{
			MockProvider->AddTextResource(
				TEXT("file:///test_notes.txt"),
				TEXT("Important test notes for the project"),
				TOptional<FString>(TEXT("test_notes")),
				TOptional<FString>(TEXT("Test Project Notes")),
				TOptional<FString>(TEXT("Test development notes and reminders")),
				TOptional<FString>(TEXT("text/plain")));

			FModelContextProtocolResourceDescriptorList List;
			MockProvider->ListResources(List);
			TestTrue("ListResources should have been called", MockProvider->bListResourcesCalled);
			TestEqual("Should have 1 descriptor", List.Num(), 1);

			TSharedPtr<const IModelContextProtocolResourceProvider> FoundProvider = List.FindResourceProvider(TEXT("file:///test_notes.txt"));
			if (TestTrue("Should find provider", FoundProvider.IsValid()))
			{
				TValueOrError<FModelContextProtocolResource, FString> Result = FoundProvider->ReadResource(TEXT("file:///test_notes.txt"));
				TestTrue("ReadResource should have been called", MockProvider->bReadResourceCalled);
				TestEqual("Last read URI should match", MockProvider->LastReadUri, TEXT("file:///test_notes.txt"));
				if (TestTrue("ReadResource should succeed", Result.HasValue()))
				{
					TSharedRef<FJsonObject> JsonObject = Result.GetValue().GetJsonObject();
					FString Text;
					TestTrue("Should have text", JsonObject->TryGetStringField(TEXT("text"), Text));
					TestEqual("Text should match", Text, TEXT("Important test notes for the project"));
				}
			}
		});

		It("should read binary resources through mock provider", [this]()
		{
			TArray<uint8> ImageData = { 0xFF, 0xD8, 0xFF, 0xE0, 0x00, 0x10, 0x4A, 0x46 };
			MockProvider->AddBinaryResource(
				TEXT("file:///test_photo.jpg"),
				ImageData,
				TOptional<FString>(TEXT("test_photo")),
				TOptional<FString>(TEXT("Test Photo")),
				TOptional<FString>(TEXT("A test photograph for binary resource validation")),
				TOptional<FString>(TEXT("image/jpeg")));

			FModelContextProtocolResourceDescriptorList List;
			MockProvider->ListResources(List);
			TestEqual("Should have 1 descriptor", List.Num(), 1);

			TValueOrError<FModelContextProtocolResource, FString> Result = MockProvider->ReadResource(TEXT("file:///test_photo.jpg"));
			if (TestTrue("ReadResource should succeed", Result.HasValue()))
			{
				TSharedRef<FJsonObject> JsonObject = Result.GetValue().GetJsonObject();
				FString Blob;
				TestTrue("Should have blob", JsonObject->TryGetStringField(TEXT("blob"), Blob));
				TestFalse("Blob should not be empty", Blob.IsEmpty());
				FString MimeType;
				if (TestTrue("Should have mimeType", JsonObject->TryGetStringField(TEXT("mimeType"), MimeType)))
				{
					TestEqual("MimeType should match", MimeType, TEXT("image/jpeg"));
				}
			}
		});

		It("should return error for unknown URI through mock provider", [this]()
		{
			TValueOrError<FModelContextProtocolResource, FString> Result = MockProvider->ReadResource(TEXT("file:///test_nonexistent.txt"));
			TestTrue("ReadResource should return error", Result.HasError());
			TestTrue("Error should mention the URI", Result.GetError().Contains(TEXT("test_nonexistent.txt")));
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
