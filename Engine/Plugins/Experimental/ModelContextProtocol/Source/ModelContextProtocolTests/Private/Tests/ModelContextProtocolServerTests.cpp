// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolServer.h"
#include "ModelContextProtocolServerAccessor.h"
#include "ModelContextProtocolSettings.h"

#include "HttpResultCallback.h"
#include "HttpServerResponse.h"
#include "Misc/AutomationTest.h"
#include "Templates/UniquePtr.h"

#if WITH_DEV_AUTOMATION_TESTS

// ---------------------------------------------------------------------------
// FModelContextProtocolServerScheduleToolsListChangedBroadcastTests
//
// Regression coverage for `ScheduleToolsListChangedBroadcast` deferring its SSE
// write to the next `Tick`, so that callers on the stack of an in-flight tool
// handler cannot reenter the HTTP connection state machine mid-write.
// ---------------------------------------------------------------------------

BEGIN_DEFINE_SPEC(FModelContextProtocolServerScheduleToolsListChangedBroadcastTests, "AI.ModelContextProtocol.Server.ScheduleToolsListChangedBroadcast", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FModelContextProtocolServerScheduleToolsListChangedBroadcastTests)

void FModelContextProtocolServerScheduleToolsListChangedBroadcastTests::Define()
{
	using namespace UE::ModelContextProtocol;

	It("must not invoke EventStreamWrite synchronously; the write must happen on the next Tick", [this]()
	{
		FModelContextProtocolServer Server;

		int32 SyncWrites = 0;
		int32 TickedWrites = 0;
		bool bTickPhase = false;

		FHttpResultCallback Spy = [&](TUniquePtr<FHttpServerResponse>&& Response)
		{
			if (bTickPhase) { ++TickedWrites; } else { ++SyncWrites; }
		};

		FServerAccessor::AddInitializedSessionWithEventStream(Server, Spy);

		Server.ScheduleToolsListChangedBroadcast();
		TestEqual("No synchronous write from ScheduleToolsListChangedBroadcast", SyncWrites, 0);
		TestEqual("No ticked write yet", TickedWrites, 0);

		bTickPhase = true;
		FServerAccessor::Tick(Server, 0.016f);
		TestEqual("Synchronous count unchanged after Tick", SyncWrites, 0);
		TestEqual("Ticked write delivered exactly once", TickedWrites, 1);

		// Subsequent ticks must not redeliver the broadcast.
		FServerAccessor::Tick(Server, 0.016f);
		TestEqual("No additional ticked writes from idle ticks", TickedWrites, 1);
	});

	It("must drain the schedule flag on Tick even when there are no sessions", [this]()
	{
		FModelContextProtocolServer Server;

		Server.ScheduleToolsListChangedBroadcast();
		TestTrue("Flag is set after scheduling", FServerAccessor::IsBroadcastScheduled(Server));

		FServerAccessor::Tick(Server, 0.016f);
		TestFalse("Flag is cleared after Tick with no sessions", FServerAccessor::IsBroadcastScheduled(Server));
	});
}

// ---------------------------------------------------------------------------
// FModelContextProtocolServerUrlPathTests
//
// Regression coverage for `FModelContextProtocolServer::StartServer` rejecting
// misconfigured URL paths before they reach `FHttpRouter::BindRoute` and trip
// the `HttpPath.IsValidPath()` assertion.
// ---------------------------------------------------------------------------

BEGIN_DEFINE_SPEC(FModelContextProtocolServerUrlPathTests, "AI.ModelContextProtocol.Server.UrlPath", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	TUniquePtr<FModelContextProtocolServer> Server;
	// Arbitrary value: every case in this spec calls StartServer with an invalid path and bails
	// before FHttpServerModule::GetHttpRouter is touched, so no listener is ever bound.
	static constexpr uint32 TestPort = 15872;
END_DEFINE_SPEC(FModelContextProtocolServerUrlPathTests)

void FModelContextProtocolServerUrlPathTests::Define()
{
	BeforeEach([this]()
	{
		Server = MakeUnique<FModelContextProtocolServer>();
	});

	AfterEach([this]()
	{
		Server.Reset();
	});

	Describe("StartServer with an invalid UrlPath", [this]()
	{
		// Each invalid-path case emits a `LogModelContextProtocol: Error: Invalid MCP server URL path ...`
		// from the validation path. That is the expected behaviour; allow-list it so the automation
		// framework doesn't flag the deliberate error log as a test failure. 
		const FString InvalidPathLogPattern = TEXT("Invalid MCP server URL path");

		It("should not start the server when UrlPath is empty", [this, InvalidPathLogPattern]()
		{
			AddExpectedError(InvalidPathLogPattern, EAutomationExpectedErrorFlags::Contains, 1);
			Server->StartServer(TestPort, TEXT(""));
			TestFalse("IsServerRunning after empty path", Server->IsServerRunning());
		});

		It("should not start the server when UrlPath lacks the leading slash", [this, InvalidPathLogPattern]()
		{
			AddExpectedError(InvalidPathLogPattern, EAutomationExpectedErrorFlags::Contains, 1);
			Server->StartServer(TestPort, TEXT("mcp"));
			TestFalse("IsServerRunning after path without leading slash", Server->IsServerRunning());
		});

		It("should not start the server when UrlPath is just root", [this, InvalidPathLogPattern]()
		{
			AddExpectedError(InvalidPathLogPattern, EAutomationExpectedErrorFlags::Contains, 1);
			Server->StartServer(TestPort, TEXT("/"));
			TestFalse("IsServerRunning after root path", Server->IsServerRunning());
		});

		It("should not start the server when UrlPath contains a disallowed character", [this, InvalidPathLogPattern]()
		{
			AddExpectedError(InvalidPathLogPattern, EAutomationExpectedErrorFlags::Contains, 1);
			Server->StartServer(TestPort, TEXT("/mcp api"));
			TestFalse("IsServerRunning after path with space", Server->IsServerRunning());
		});

		It("should not start the server when UrlPath contains a comma", [this, InvalidPathLogPattern]()
		{
			AddExpectedError(InvalidPathLogPattern, EAutomationExpectedErrorFlags::Contains, 1);
			Server->StartServer(TestPort, TEXT("/mcp,v1"));
			TestFalse("IsServerRunning after path with comma", Server->IsServerRunning());
		});
	});
}

// ---------------------------------------------------------------------------
// FModelContextProtocolSettingsEnforceValidServerUrlPathTests
//
// Direct coverage for `UModelContextProtocolSettings::EnforceValidServerUrlPath`,
// the shared sanitiser called from the editor-side `PostEditChangeProperty`
// guard. Tests the logic without going through the UObject property machinery.
// ---------------------------------------------------------------------------

BEGIN_DEFINE_SPEC(FModelContextProtocolSettingsEnforceValidServerUrlPathTests, "AI.ModelContextProtocol.Settings.EnforceValidServerUrlPath", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FModelContextProtocolSettingsEnforceValidServerUrlPathTests)

void FModelContextProtocolSettingsEnforceValidServerUrlPathTests::Define()
{
	const FString InvalidPathLogPattern = TEXT("not a valid HTTP path");
	const FString DefaultPath = UE::ModelContextProtocol::DefaultServerUrlPath;

	Describe("with an invalid path", [this, InvalidPathLogPattern, DefaultPath]()
	{
		It("should replace an empty path with the default", [this, InvalidPathLogPattern, DefaultPath]()
		{
			AddExpectedError(InvalidPathLogPattern, EAutomationExpectedErrorFlags::Contains, 1);
			FString Path = TEXT("");
			UModelContextProtocolSettings::EnforceValidServerUrlPath(Path);
			TestEqual("Empty path reverted to default", Path, DefaultPath);
		});

		It("should replace a path missing the leading slash with the default", [this, InvalidPathLogPattern, DefaultPath]()
		{
			AddExpectedError(InvalidPathLogPattern, EAutomationExpectedErrorFlags::Contains, 1);
			FString Path = TEXT("mcp");
			UModelContextProtocolSettings::EnforceValidServerUrlPath(Path);
			TestEqual("Path without leading slash reverted to default", Path, DefaultPath);
		});

		It("should replace the root path with the default", [this, InvalidPathLogPattern, DefaultPath]()
		{
			AddExpectedError(InvalidPathLogPattern, EAutomationExpectedErrorFlags::Contains, 1);
			FString Path = TEXT("/");
			UModelContextProtocolSettings::EnforceValidServerUrlPath(Path);
			TestEqual("Root path reverted to default", Path, DefaultPath);
		});

		It("should replace a path with a disallowed character with the default", [this, InvalidPathLogPattern, DefaultPath]()
		{
			AddExpectedError(InvalidPathLogPattern, EAutomationExpectedErrorFlags::Contains, 1);
			FString Path = TEXT("/mcp api");
			UModelContextProtocolSettings::EnforceValidServerUrlPath(Path);
			TestEqual("Path with space reverted to default", Path, DefaultPath);
		});
	});

	Describe("with a valid path", [this]()
	{
		It("should leave '/mcp' unchanged", [this]()
		{
			FString Path = TEXT("/mcp");
			UModelContextProtocolSettings::EnforceValidServerUrlPath(Path);
			TestEqual("Valid '/mcp' unchanged", Path, TEXT("/mcp"));
		});

		It("should leave a nested valid path unchanged", [this]()
		{
			FString Path = TEXT("/api/mcp");
			UModelContextProtocolSettings::EnforceValidServerUrlPath(Path);
			TestEqual("Valid nested path unchanged", Path, TEXT("/api/mcp"));
		});
	});
}

#endif
