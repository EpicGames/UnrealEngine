// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolSession.h"
#include "Dom/JsonValue.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolSessionTests, "AI.ModelContextProtocol.Session", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FModelContextProtocolSessionTests)

void FModelContextProtocolSessionTests::Define()
{
	Describe("FModelContextProtocolToolRequestId", [this]()
	{
		It("should be invalid when default-constructed", [this]()
		{
			FModelContextProtocolToolRequestId RequestId;
			TestFalse("Default-constructed should be invalid", RequestId.IsValid());
		});

		It("should be valid when constructed with a valid FJsonValue", [this]()
		{
			TSharedPtr<FJsonValue> JsonValue = MakeShared<FJsonValueString>(TEXT("request-1"));
			FModelContextProtocolToolRequestId RequestId(JsonValue);
			TestTrue("Should be valid", RequestId.IsValid());
		});

		It("should compare equal for identical string request IDs via FJsonValue::CompareEqual", [this]()
		{
			TSharedPtr<FJsonValue> ValueA = MakeShared<FJsonValueString>(TEXT("request-1"));
			TSharedPtr<FJsonValue> ValueB = MakeShared<FJsonValueString>(TEXT("request-1"));
			FModelContextProtocolToolRequestId IdA(ValueA);
			FModelContextProtocolToolRequestId IdB(ValueB);
			// operator== is not DLL-exported, so test the underlying comparison directly
			TestTrue("Same string IDs should compare equal",
				IdA.IsValid() && IdB.IsValid() && FJsonValue::CompareEqual(*IdA.RequestId, *IdB.RequestId));
		});

		It("should compare equal for identical numeric request IDs via FJsonValue::CompareEqual", [this]()
		{
			TSharedPtr<FJsonValue> ValueA = MakeShared<FJsonValueNumber>(42);
			TSharedPtr<FJsonValue> ValueB = MakeShared<FJsonValueNumber>(42);
			FModelContextProtocolToolRequestId IdA(ValueA);
			FModelContextProtocolToolRequestId IdB(ValueB);
			TestTrue("Same numeric IDs should compare equal",
				IdA.IsValid() && IdB.IsValid() && FJsonValue::CompareEqual(*IdA.RequestId, *IdB.RequestId));
		});

		It("should not compare equal for different IDs", [this]()
		{
			TSharedPtr<FJsonValue> ValueA = MakeShared<FJsonValueString>(TEXT("request-1"));
			TSharedPtr<FJsonValue> ValueB = MakeShared<FJsonValueString>(TEXT("request-2"));
			FModelContextProtocolToolRequestId IdA(ValueA);
			FModelContextProtocolToolRequestId IdB(ValueB);
			TestFalse("Different IDs should not compare equal",
				FJsonValue::CompareEqual(*IdA.RequestId, *IdB.RequestId));
		});

		It("should produce consistent hash values for equal IDs via AsString", [this]()
		{
			TSharedPtr<FJsonValue> ValueA = MakeShared<FJsonValueString>(TEXT("request-1"));
			TSharedPtr<FJsonValue> ValueB = MakeShared<FJsonValueString>(TEXT("request-1"));
			FModelContextProtocolToolRequestId IdA(ValueA);
			FModelContextProtocolToolRequestId IdB(ValueB);
			// GetTypeHash is not DLL-exported, so test the underlying hash logic directly
			TestEqual("Hashes should match for equal IDs",
				GetTypeHash(IdA.RequestId->AsString()), GetTypeHash(IdB.RequestId->AsString()));
		});
	});

	Describe("FModelContextProtocolSession", [this]()
	{
		It("should default to Initializing status", [this]()
		{
			FModelContextProtocolSession Session;
			TestEqual("Default status should be Initializing",
				Session.Status, EModelContextProtocolSessionStatus::Initializing);
		});

		It("should start with empty ActiveRequests", [this]()
		{
			FModelContextProtocolSession Session;
			TestEqual("ActiveRequests should be empty", Session.ActiveRequests.Num(), 0);
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
