// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocol.h"
#include "IModelContextProtocolModule.h"
#include "Mocks/MockModelContextProtocolTool.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolToolNameTests, "AI.ModelContextProtocol.ToolNames", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FModelContextProtocolToolNameTests)

void FModelContextProtocolToolNameTests::Define()
{
	using namespace UE::ModelContextProtocol;

	Describe("ValidateToolName", [this]()
	{
		It("should accept simple alphanumeric names", [this]()
		{
			TestEqual("getUser", ValidateToolName(TEXT("getUser")), EToolNameValidation::Valid);
			TestEqual("DATA_EXPORT_v2", ValidateToolName(TEXT("DATA_EXPORT_v2")), EToolNameValidation::Valid);
		});

		It("should accept names with dots", [this]()
		{
			TestEqual("admin.tools.list", ValidateToolName(TEXT("admin.tools.list")), EToolNameValidation::Valid);
			TestEqual("dotted PascalCase", ValidateToolName(TEXT("ToolsetRegistry.EditorAppToolset.GetSelectedActors")), EToolNameValidation::Valid);
		});

		It("should accept names with hyphens and underscores", [this]()
		{
			TestEqual("my-tool", ValidateToolName(TEXT("my-tool")), EToolNameValidation::Valid);
			TestEqual("my_tool", ValidateToolName(TEXT("my_tool")), EToolNameValidation::Valid);
			TestEqual("my-tool_v2.0", ValidateToolName(TEXT("my-tool_v2.0")), EToolNameValidation::Valid);
		});

		It("should accept names with snake_case and dots", [this]()
		{
			TestEqual("snake_case.dotted", ValidateToolName(TEXT("toolset_registry.toolsets.core.actor.ActorTools.get_label")), EToolNameValidation::Valid);
		});

		It("should reject empty names", [this]()
		{
			TestEqual("empty string", ValidateToolName(TEXT("")), EToolNameValidation::Empty);
		});

		It("should reject names exceeding 128 characters", [this]()
		{
			FString LongName;
			for (int32 Index = 0; Index < 129; ++Index)
			{
				LongName += TEXT("a");
			}
			TestEqual("129 chars", ValidateToolName(LongName), EToolNameValidation::ExceedsMaxLength);
		});

		It("should accept names at exactly 128 characters", [this]()
		{
			FString MaxName;
			for (int32 Index = 0; Index < 128; ++Index)
			{
				MaxName += TEXT("a");
			}
			TestEqual("128 chars", ValidateToolName(MaxName), EToolNameValidation::Valid);
		});

		It("should reject names with spaces", [this]()
		{
			TestEqual("my tool", ValidateToolName(TEXT("my tool")), EToolNameValidation::InvalidCharacters);
		});

		It("should reject names with special characters", [this]()
		{
			TestEqual("my@tool", ValidateToolName(TEXT("my@tool")), EToolNameValidation::InvalidCharacters);
			TestEqual("my/tool", ValidateToolName(TEXT("my/tool")), EToolNameValidation::InvalidCharacters);
			TestEqual("my:tool", ValidateToolName(TEXT("my:tool")), EToolNameValidation::InvalidCharacters);
			TestEqual("my,tool", ValidateToolName(TEXT("my,tool")), EToolNameValidation::InvalidCharacters);
		});
	});

	Describe("AddTool name validation", [this]()
	{
		It("should reject tools with empty names", [this]()
		{
			TSharedPtr<FMockModelContextProtocolTool> MockTool = MakeShared<FMockModelContextProtocolTool>();
			MockTool->Name = TEXT("");

			AddExpectedError(TEXT("Rejecting tool with empty name"), EAutomationExpectedErrorFlags::Contains);

			IModelContextProtocolModule& Module = IModelContextProtocolModule::GetChecked();
			TestFalse("Empty-named tool should be rejected", Module.AddTool(MockTool.ToSharedRef()));
		});

		It("should still register tools with invalid characters", [this]()
		{
			TSharedPtr<FMockModelContextProtocolTool> MockTool = MakeShared<FMockModelContextProtocolTool>();
			MockTool->Name = TEXT("my invalid tool name");

			IModelContextProtocolModule& Module = IModelContextProtocolModule::GetChecked();
			const bool bAdded = Module.AddTool(MockTool.ToSharedRef());
			TestTrue("Tool should still be registered despite non-compliant name", bAdded);

			if (bAdded)
			{
				Module.RemoveTool(MockTool.ToSharedRef());
			}
		});

		It("should warn for tool names exceeding 128 characters", [this]()
		{
			TSharedPtr<FMockModelContextProtocolTool> MockTool = MakeShared<FMockModelContextProtocolTool>();
			FString LongName;
			for (int32 Index = 0; Index < 129; ++Index)
			{
				LongName += TEXT("a");
			}
			MockTool->Name = LongName;

			AddExpectedError(TEXT("exceeds MCP spec limit"), EAutomationExpectedErrorFlags::Contains);

			IModelContextProtocolModule& Module = IModelContextProtocolModule::GetChecked();
			const bool bAdded = Module.AddTool(MockTool.ToSharedRef());
			TestTrue("Tool should still be registered despite length violation", bAdded);

			if (bAdded)
			{
				Module.RemoveTool(MockTool.ToSharedRef());
			}
		});

		It("should register tools with compliant names without warning", [this]()
		{
			TSharedPtr<FMockModelContextProtocolTool> MockTool = MakeShared<FMockModelContextProtocolTool>();
			MockTool->Name = TEXT("valid.tool-name_v2");

			IModelContextProtocolModule& Module = IModelContextProtocolModule::GetChecked();
			const bool bAdded = Module.AddTool(MockTool.ToSharedRef());
			TestTrue("Tool should be registered", bAdded);

			if (bAdded)
			{
				Module.RemoveTool(MockTool.ToSharedRef());
			}
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
