// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolToolResults.h"
#include "ModelContextProtocolResources.h"
#include "ModelContextProtocolSession.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Misc/AutomationTest.h"
#include "Misc/Base64.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolToolResultsTests, "AI.ModelContextProtocol.ToolResults", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FModelContextProtocolToolResultsTests)

void FModelContextProtocolToolResultsTests::Define()
{
	using namespace UE::ModelContextProtocol;

	Describe("MakeTextResult", [this]()
	{
		It("should produce content array with text object", [this]()
		{
			FModelContextProtocolToolResult Result = MakeTextResult(TEXT("hello"));
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content array", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestEqual("Content should have 1 element", ContentArray->Num(), 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			FString Type;
			TestTrue("Should have type field", (*ContentObject)->TryGetStringField(TEXT("type"), Type));
			TestEqual("Type should be text", Type, TEXT("text"));
			FString Text;
			TestTrue("Should have text field", (*ContentObject)->TryGetStringField(TEXT("text"), Text));
			TestEqual("Text should match", Text, TEXT("hello"));
		});

		It("should handle empty strings", [this]()
		{
			FModelContextProtocolToolResult Result = MakeTextResult(TEXT(""));
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content array", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestTrue("Should have at least 1 element", ContentArray->Num() >= 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			FString Text;
			TestTrue("Should have text field", (*ContentObject)->TryGetStringField(TEXT("text"), Text));
			TestEqual("Text should be empty", Text, TEXT(""));
		});

		It("should handle FString input", [this]()
		{
			FString Input = TEXT("fstring test");
			FModelContextProtocolToolResult Result = MakeTextResult(Input);
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content array", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestTrue("Should have at least 1 element", ContentArray->Num() >= 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			FString Text;
			TestTrue("Should have text field", (*ContentObject)->TryGetStringField(TEXT("text"), Text));
			TestEqual("Text should match FString input", Text, TEXT("fstring test"));
		});
	});

	Describe("MakeErrorResult", [this]()
	{
		It("should set isError to true", [this]()
		{
			FModelContextProtocolToolResult Result = MakeErrorResult(TEXT("something went wrong"));
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			bool bIsError = false;
			TestTrue("Should have isError field", Result.JsonObject->TryGetBoolField(TEXT("isError"), bIsError));
			TestTrue("isError should be true", bIsError);
		});

		It("should include error message in content text", [this]()
		{
			FModelContextProtocolToolResult Result = MakeErrorResult(TEXT("test error"));
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestTrue("Should have at least 1 element", ContentArray->Num() >= 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			FString Text;
			TestTrue("Should have text field", (*ContentObject)->TryGetStringField(TEXT("text"), Text));
			TestEqual("Should contain error message", Text, TEXT("test error"));
		});
	});

	Describe("MakeImageResult (base64 overloads)", [this]()
	{
		It("should base64-encode raw data with correct type", [this]()
		{
			TArray<uint8> Data = { 0xFF, 0xD8, 0xFF, 0xE0 };
			FModelContextProtocolToolResult Result = MakeImageResult(TEXT("image/jpeg"), TArrayView<uint8>(Data));
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestTrue("Should have at least 1 element", ContentArray->Num() >= 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			FString Type;
			TestTrue("Should have type", (*ContentObject)->TryGetStringField(TEXT("type"), Type));
			TestEqual("Type should be image", Type, TEXT("image"));
			FString MimeType;
			TestTrue("Should have mimeType", (*ContentObject)->TryGetStringField(TEXT("mimeType"), MimeType));
			TestEqual("MimeType should match", MimeType, TEXT("image/jpeg"));
			FString EncodedData;
			TestTrue("Should have data field", (*ContentObject)->TryGetStringField(TEXT("data"), EncodedData));
			TestFalse("Data should not be empty", EncodedData.IsEmpty());
		});

		It("should pass through pre-encoded base64 data", [this]()
		{
			FString PreEncoded = TEXT("AQIDBA==");
			FModelContextProtocolToolResult Result = MakeImageResult(TEXT("image/png"), FString(PreEncoded));
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestTrue("Should have at least 1 element", ContentArray->Num() >= 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			FString Data;
			TestTrue("Should have data", (*ContentObject)->TryGetStringField(TEXT("data"), Data));
			TestEqual("Should match pre-encoded data", Data, PreEncoded);
		});

		It("should include audience annotation for User", [this]()
		{
			TArray<uint8> Data = { 0x01 };
			FModelContextProtocolToolResult Result = MakeImageResult(TEXT("image/png"), TArrayView<uint8>(Data), EModelContextProtocolAudience::User);
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestTrue("Should have at least 1 element", ContentArray->Num() >= 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			const TSharedPtr<FJsonObject>* Annotations;
			if (!TestTrue("Should have annotations", (*ContentObject)->TryGetObjectField(TEXT("annotations"), Annotations))) { return; }
			const TArray<TSharedPtr<FJsonValue>>* AudienceArray;
			if (!TestTrue("Should have audience array", (*Annotations)->TryGetArrayField(TEXT("audience"), AudienceArray))) { return; }
			bool bFoundUser = false;
			for (const auto& Value : *AudienceArray)
			{
				if (Value->AsString() == TEXT("user"))
				{
					bFoundUser = true;
				}
			}
			TestTrue("Should contain user audience", bFoundUser);
		});

		It("should include both user and assistant audiences for All", [this]()
		{
			TArray<uint8> Data = { 0x01 };
			FModelContextProtocolToolResult Result = MakeImageResult(TEXT("image/png"), TArrayView<uint8>(Data), EModelContextProtocolAudience::All);
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestTrue("Should have at least 1 element", ContentArray->Num() >= 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			const TSharedPtr<FJsonObject>* Annotations;
			if (!TestTrue("Should have annotations", (*ContentObject)->TryGetObjectField(TEXT("annotations"), Annotations))) { return; }
			const TArray<TSharedPtr<FJsonValue>>* AudienceArray;
			if (!TestTrue("Should have audience array", (*Annotations)->TryGetArrayField(TEXT("audience"), AudienceArray))) { return; }
			bool bFoundUser = false;
			bool bFoundAssistant = false;
			for (const auto& Value : *AudienceArray)
			{
				if (Value->AsString() == TEXT("user")) { bFoundUser = true; }
				if (Value->AsString() == TEXT("assistant")) { bFoundAssistant = true; }
			}
			TestTrue("Should contain user audience", bFoundUser);
			TestTrue("Should contain assistant audience", bFoundAssistant);
		});
	});

	Describe("MakeAudioResult (base64 overloads)", [this]()
	{
		It("should base64-encode raw audio data with correct type and mimeType", [this]()
		{
			TArray<uint8> Data = { 0x00, 0x01, 0x02, 0x03 };
			FModelContextProtocolToolResult Result = MakeAudioResult(TEXT("audio/wav"), TArrayView<uint8>(Data));
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestTrue("Should have at least 1 element", ContentArray->Num() >= 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			FString Type;
			TestTrue("Should have type", (*ContentObject)->TryGetStringField(TEXT("type"), Type));
			TestEqual("Type should be audio", Type, TEXT("audio"));
			FString MimeType;
			TestTrue("Should have mimeType", (*ContentObject)->TryGetStringField(TEXT("mimeType"), MimeType));
			TestEqual("MimeType should match", MimeType, TEXT("audio/wav"));
		});

		It("should pass through pre-encoded audio data", [this]()
		{
			FString PreEncoded = TEXT("AQIDBA==");
			FModelContextProtocolToolResult Result = MakeAudioResult(TEXT("audio/ogg"), FString(PreEncoded));
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestTrue("Should have at least 1 element", ContentArray->Num() >= 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			FString Data;
			TestTrue("Should have data", (*ContentObject)->TryGetStringField(TEXT("data"), Data));
			TestEqual("Should match pre-encoded data", Data, PreEncoded);
		});
	});

	Describe("MakeResourceLinkResult / MakeResourceLinksResult", [this]()
	{
		It("should produce resource link content with URI and metadata", [this]()
		{
			FModelContextProtocolResourceDescriptor Descriptor(
				TEXT("file:///test_settings.json"),
				TOptional<FString>(TEXT("test_settings")),
				TOptional<FString>(TEXT("Test Settings")),
				TOptional<FString>(TEXT("A test settings file for resource link validation")),
				TOptional<FString>(TEXT("application/json")));
			FModelContextProtocolToolResult Result = MakeResourceLinkResult(Descriptor);
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			if (!TestEqual("Should have 1 element", ContentArray->Num(), 1)) { return; }
			const TSharedPtr<FJsonObject>* ContentObject;
			if (!TestTrue("Element should be object", (*ContentArray)[0]->TryGetObject(ContentObject))) { return; }
			FString Type;
			TestTrue("Should have type", (*ContentObject)->TryGetStringField(TEXT("type"), Type));
			TestEqual("Type should be resource_link", Type, TEXT("resource_link"));
		});

		It("should produce multiple resource links with varied descriptors", [this]()
		{
			TArray<FModelContextProtocolResourceDescriptor> Descriptors;
			Descriptors.Emplace(TEXT("file:///test_changelog.md"), TOptional<FString>(TEXT("test_changelog")), TOptional<FString>(TEXT("Test Changelog")), TOptional<FString>(TEXT("A test changelog for resource link validation")), TOptional<FString>(TEXT("text/markdown")));
			Descriptors.Emplace(TEXT("file:///test_metrics.csv"), TOptional<FString>(TEXT("test_metrics")), TOptional<FString>(TEXT("Test Metrics Export")), TOptional<FString>(TEXT("A test metrics dataset for resource link validation")), TOptional<FString>(TEXT("text/csv")));
			Descriptors.Emplace(TEXT("file:///test_icon.png"));
			FModelContextProtocolToolResult Result = MakeResourceLinksResult(Descriptors);
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			TestEqual("Should have 3 elements", ContentArray->Num(), 3);
		});

		It("should handle empty descriptor array", [this]()
		{
			TArray<FModelContextProtocolResourceDescriptor> Descriptors;
			FModelContextProtocolToolResult Result = MakeResourceLinksResult(Descriptors);
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			TestEqual("Should have 0 elements", ContentArray->Num(), 0);
		});
	});

	Describe("MakeStructuredContentResult", [this]()
	{
		It("should produce structuredContent from TSharedPtr<FJsonValue> passthrough", [this]()
		{
			TSharedPtr<FJsonObject> InnerObject = MakeShared<FJsonObject>();
			InnerObject->SetStringField(TEXT("key"), TEXT("value"));
			TSharedPtr<FJsonValue> JsonValue = MakeShared<FJsonValueObject>(InnerObject);
			FModelContextProtocolToolResult Result = MakeStructuredContentResult(JsonValue);
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			TestTrue("Should have structuredContent field", Result.JsonObject->HasField(TEXT("structuredContent")));
		});

		It("should include text fallback in content array", [this]()
		{
			TSharedPtr<FJsonObject> InnerObject = MakeShared<FJsonObject>();
			InnerObject->SetStringField(TEXT("key"), TEXT("value"));
			TSharedPtr<FJsonValue> JsonValue = MakeShared<FJsonValueObject>(InnerObject);
			FModelContextProtocolToolResult Result = MakeStructuredContentResult(JsonValue);
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (!TestTrue("Should have content array", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray))) { return; }
			TestTrue("Content should not be empty", ContentArray->Num() > 0);
		});

		It("should produce structuredContent from UStruct pointer", [this]()
		{
			FModelContextProtocolClientInfo ClientInfo;
			ClientInfo.Name = TEXT("TestClient");
			ClientInfo.Title = TEXT("Test Client Application");
			ClientInfo.Version = TEXT("1.0.0");

			FModelContextProtocolToolResult Result = MakeStructuredContentResult(
				FModelContextProtocolClientInfo::StaticStruct(), &ClientInfo);
			if (!TestTrue("Result should have JSON", Result.JsonObject.IsValid())) { return; }
			TestTrue("Should have structuredContent field", Result.JsonObject->HasField(TEXT("structuredContent")));

			const TSharedPtr<FJsonObject>* StructuredContent;
			if (TestTrue("structuredContent should be object", Result.JsonObject->TryGetObjectField(TEXT("structuredContent"), StructuredContent)))
			{
				FString Name;
				bool bHasName = (*StructuredContent)->TryGetStringField(TEXT("Name"), Name) || (*StructuredContent)->TryGetStringField(TEXT("name"), Name);
				if (bHasName)
				{
					TestEqual("Struct Name should match", Name, TEXT("TestClient"));
				}
			}

			const TArray<TSharedPtr<FJsonValue>>* ContentArray;
			if (TestTrue("Should have content array", Result.JsonObject->TryGetArrayField(TEXT("content"), ContentArray)))
			{
				TestTrue("Content should not be empty", ContentArray->Num() > 0);
			}
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
