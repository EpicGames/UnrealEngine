// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelContextProtocolUtils.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FModelContextProtocolUtilsTests, "AI.ModelContextProtocol.Utils", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FModelContextProtocolUtilsTests)

void FModelContextProtocolUtilsTests::Define()
{
	using namespace UE::ModelContextProtocol;

	Describe("SafeConvertRelativePathToFull", [this]()
	{
		It("should join base path and filename correctly", [this]()
		{
			FString OutFilePath;
			const bool bResult = SafeConvertRelativePathToFull(TEXT("C:/Base"), TEXT("File.txt"), OutFilePath);
			TestTrue("Should succeed", bResult);
			TestTrue("Path should contain filename", OutFilePath.Contains(TEXT("File.txt")));
			TestTrue("Path should be under base", OutFilePath.Contains(TEXT("Base")));
		});

		It("should succeed for paths under the base directory", [this]()
		{
			FString OutFilePath;
			const bool bResult = SafeConvertRelativePathToFull(TEXT("C:/Base"), TEXT("Sub/File.txt"), OutFilePath);
			TestTrue("Should succeed for subdirectory path", bResult);
			TestFalse("OutFilePath should not be empty", OutFilePath.IsEmpty());
		});

		It("should fail for directory traversal attacks", [this]()
		{
			FString OutFilePath;
			const bool bResult = SafeConvertRelativePathToFull(TEXT("C:/Base"), TEXT("../../Escape/File.txt"), OutFilePath);
			TestFalse("Should fail for traversal", bResult);
		});

		It("should clear OutFilePath on failure", [this]()
		{
			FString OutFilePath = TEXT("should be cleared");
			SafeConvertRelativePathToFull(TEXT("C:/Base"), TEXT("../../Escape/File.txt"), OutFilePath);
			TestTrue("OutFilePath should be empty after failure", OutFilePath.IsEmpty());
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
